// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class AnimationModifiers : ModuleRules
{
	public AnimationModifiers(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] 
            {
                "Core",
                "CoreUObject",
                "Engine",
                "InputCore",
                "Slate",
                "SlateCore",
                "UnrealEd",
                "MainFrame",
                "PropertyEditor",
                "AssetTools",
                "ClassViewer",
                "AssetRegistry",
                "AnimationBlueprintLibrary",
                "DeveloperSettings",
                "ContentBrowser",
                "ToolMenus"
            }
		);
    }
}
