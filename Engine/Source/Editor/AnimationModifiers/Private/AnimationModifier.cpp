// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationModifier.h"
#include "AnimationModifiersAssetUserData.h"
#include "AnimationModifierHelpers.h"

#include "Algo/Transform.h"
#include "Animation/AnimData/IAnimationDataController.h"
#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetViewUtils.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Editor/Transactor.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/MessageDialog.h"
#include "Misc/OutputDeviceRedirector.h"
#include "ModifierOutputFilter.h"
#include "Modules/ModuleManager.h"
#include "ScopedTransaction.h"
#include "Serialization/Archive.h"
#include "Templates/Casts.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectKey.h"
#include "UObject/Package.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/Script.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationModifier)

#define LOCTEXT_NAMESPACE "AnimationModifier"

const FName UAnimationModifier::AnimationModifiersTag = TEXT("AnimationModifierList");

UAnimationModifier::UAnimationModifier()
{
}

void UAnimationModifier::ApplyToAnimationSequence(UAnimSequence* AnimSequence) const
{
	FEditorScriptExecutionGuard ScriptGuard;

	checkf(AnimSequence, TEXT("Invalid Animation Sequence supplied"));

	//Make sure Anim sequence compression is done
	AnimSequence->WaitOnExistingCompression();

	USkeleton* Skeleton = AnimSequence->GetSkeleton();
	checkf(Skeleton, TEXT("Invalid Skeleton for supplied Animation Sequence"));

	// Filter to check for warnings / errors thrown from animation blueprint library (rudimentary approach for now)
	FCategoryLogOutputFilter OutputLog;
	OutputLog.SetAutoEmitLineTerminator(true);
	OutputLog.AddCategoryName("LogAnimationBlueprintLibrary");
	OutputLog.AddCategoryName("LogAnimation");

	GLog->AddOutputDevice(&OutputLog);

	bool AppliedModifierOuterWasCreated = false;
	UObject* AppliedModifierOuter = AnimSequence->GetAssetUserData<UAnimationModifiersAssetUserData>();
	if (!AppliedModifierOuter)
	{
		AppliedModifierOuterWasCreated = true;
		AppliedModifierOuter = FAnimationModifierHelpers::RetrieveOrCreateModifierUserData(AnimSequence);
	}
	// Using explicit name may cause duplicate path name
	UAnimationModifier* AppliedModifier = DuplicateObject(this, AppliedModifierOuter);
	// Set the revision guid on applied modifier to latest
	AppliedModifier->RevisionGuid = GetLatestRevisionGuid();
	
	AnimSequence->Modify();

	FTransaction AnimationDataTransaction;
	AnimationDataTransaction.SaveObject(AnimSequence);
	AnimationDataTransaction.SaveObject(Skeleton);

	{
		IAnimationDataModel::FEvaluationAndModificationLock EvaluationAndModificationLock(*AnimSequence->GetDataModelInterface());
		// Group the OnRevert & OnApply operation into one Bracket to reduce compression request
		IAnimationDataController::FScopedBracket ScopedBracket(AnimSequence->GetController(), LOCTEXT("ApplyModifierBracket", "Applying Animation Modifier"));

		/** In case this modifier has been previously applied, revert it using the serialised out version at the time */
		if (UAnimationModifier* PreviouslyAppliedModifier = GetAppliedModifier(AnimSequence))
		{
			// Save the applied modifier as well
			AnimationDataTransaction.SaveObject(PreviouslyAppliedModifier);
			PreviouslyAppliedModifier->ExecuteOnRevert(AnimSequence);
		}

		{
			/** Reverting and applying, populates the log with possible warnings and or errors to notify the user about */
			AppliedModifier->ExecuteOnApply(AnimSequence);
		}
	}

	GLog->RemoveOutputDevice(&OutputLog);

	// Check if warnings or errors have occurred and show dialog to user to inform them about this
	const bool bWarnings = OutputLog.ContainsWarnings();
	const bool bErrors = OutputLog.ContainsErrors();
	bool bShouldRevert = bErrors;
	
	// If errors have occured - prompt user with OK and revert
	static const FText MessageTitle = LOCTEXT("ModifierDialogTitle", "Modifier has generated errors.");
	if (bErrors)
	{
		const FText ErrorMessage = FText::FormatOrdered(LOCTEXT("ModifierErrorDescription", "Modifier: {0}\nAsset: {1}\n{2}\nResolve the errors before trying to apply again."),
			FText::FromString(GetClass()->GetPathName()),
			FText::FromString(CurrentAnimSequence->GetPathName()),
			FText::FromString(OutputLog));
		UE::Anim::FApplyModifiersScope::HandleError(this, ErrorMessage, MessageTitle);
	}

	// If _only_ warnings have occured - check if user has previously said YesAll / NoAll and process the result
	if (bWarnings && !bShouldRevert)
	{
		const FText WarningMessage = FText::FormatOrdered(LOCTEXT("ModifierWarningDescription", "Modifier: {0}\nAsset: {1}\n{2}\nAre you sure you want to apply it?"),
			FText::FromString(GetClass()->GetPathName()),
			FText::FromString(CurrentAnimSequence->GetPathName()),
			FText::FromString(OutputLog));
		bShouldRevert = !UE::Anim::FApplyModifiersScope::HandleWarning(this, WarningMessage, MessageTitle);
	}
	
	// Revert changes if necessary, otherwise post edit and refresh animation data
	if (bShouldRevert)
	{
		IAnimationDataModel::FEvaluationAndModificationLock EvaluationAndModificationLock(*AnimSequence->GetDataModelInterface());
		
		AnimationDataTransaction.BeginOperation();
		AnimationDataTransaction.Apply();
		AnimationDataTransaction.EndOperation();
		AnimSequence->RefreshCacheData();
		AppliedModifier->MarkAsGarbage();
		if (AppliedModifierOuterWasCreated)
		{
			// The animation sequence's asset user data array should have been reverted by the transaction
			AppliedModifierOuter->MarkAsGarbage();
		}
	}
	else
	{
		SetAppliedModifier(AnimSequence, AppliedModifier);

		if (Skeleton->GetPackage()->IsDirty())
		{
			Skeleton->PostEditChange();
		}
		if (AnimSequence->GetPackage()->IsDirty())
		{
			AnimSequence->PostEditChange();
			AnimSequence->RefreshCacheData();
		}
	}
}

void UAnimationModifier::UpdateCompressedAnimationData()
{
	if (CurrentAnimSequence->IsCompressedDataOutOfDate())
	{
		CurrentAnimSequence->CacheDerivedDataForCurrentPlatform();
	}
}

void UAnimationModifier::RevertFromAnimationSequence(class UAnimSequence* AnimSequence) const
{
	FEditorScriptExecutionGuard ScriptGuard;

	checkf(AnimSequence, TEXT("Invalid Animation Sequence supplied"));
	USkeleton* Skeleton = AnimSequence->GetSkeleton();
	checkf(Skeleton, TEXT("Invalid Skeleton for supplied Animation Sequence"));
	/** Can only revert if previously applied, which means there should be a previous modifier */
	UAnimationModifier* PreviouslyAppliedModifier = FindAndRemoveAppliedModifier(AnimSequence);

	// Backward compatibility
	// Is PreviouslyAppliedModifier owned by InAnimationSequence
	// Modifier on Skeleton read from legacy version may _share_ the PreviouslyAppliedModifier
	bool AppliedModifierOwnedByAnimation = true;
	if (!PreviouslyAppliedModifier)
	{
		PreviouslyAppliedModifier = GetLegacyPreviouslyAppliedModifierForModifierOnSkeleton();
		if (PreviouslyAppliedModifier)
		{
			AppliedModifierOwnedByAnimation = false;
		}
	}

	if (PreviouslyAppliedModifier)
	{
		// Transact the modifier to prevent instance variables/data to change during reverting
		{
			IAnimationDataController::FScopedBracket ScopedBracket(AnimSequence->GetController(), LOCTEXT("ApplyModifierBracket", "Applying Animation Modifier"));	
			PreviouslyAppliedModifier->ExecuteOnRevert(AnimSequence);
		}

		if (AppliedModifierOwnedByAnimation)
		{
			PreviouslyAppliedModifier->MarkAsGarbage();
		}

		if (Skeleton->GetPackage()->IsDirty())
		{
			Skeleton->PostEditChange();
		}
		if (AnimSequence->GetPackage()->IsDirty())
		{
			AnimSequence->PostEditChange();
			AnimSequence->RefreshCacheData();
		}
	}
}

bool UAnimationModifier::CanRevert(IInterface_AssetUserData* AssetUserDataInterface) const
{
	return GetAppliedModifier(AssetUserDataInterface) != nullptr;
}

UAnimationModifier* UAnimationModifier::GetAppliedModifier(IInterface_AssetUserData* AssetUserDataInterface) const
{
	if (UAnimationModifiersAssetUserData* AssetData = AssetUserDataInterface->GetAssetUserData<UAnimationModifiersAssetUserData>())
	{
		TObjectPtr<UAnimationModifier>* AppliedModifier = AssetData->AppliedModifiers.Find(this);
		if (AppliedModifier)
		{
			return AppliedModifier->Get();
		}
	}

	// Backward compatibility
	// Read the shared applied instance for modifier on skeleton ready from legacy version
	if (UAnimationModifier* LegacyAppliedModifierOnSkeleton = GetLegacyPreviouslyAppliedModifierForModifierOnSkeleton())
	{
		return LegacyAppliedModifierOnSkeleton;
	}

	return nullptr;
}

UAnimationModifier* UAnimationModifier::FindAndRemoveAppliedModifier(TScriptInterface<IInterface_AssetUserData> AssetUserDataInterface) const
{
	if (UAnimationModifiersAssetUserData* AssetData = AssetUserDataInterface->GetAssetUserData<UAnimationModifiersAssetUserData>())
	{
		TObjectPtr<UAnimationModifier> AppliedModifier = nullptr;
		AssetData->AppliedModifiers.RemoveAndCopyValue(this, AppliedModifier);
		if (AppliedModifier) 
		{
			AssetData->Modify();
			AssetUserDataInterface.GetObject()->Modify();
		}
		return AppliedModifier.Get();
	}
	return nullptr;
}

void UAnimationModifier::SetAppliedModifier(TScriptInterface<IInterface_AssetUserData> AssetUserDataInterface, UAnimationModifier* AppliedModifier) const
{
	if (UAnimationModifiersAssetUserData* AssetData = FAnimationModifierHelpers::RetrieveOrCreateModifierUserData(AssetUserDataInterface))
	{
		TObjectPtr<UAnimationModifier>& Modifier = AssetData->AppliedModifiers.FindOrAdd(this);

		if (Modifier)
		{
			Modifier->MarkAsGarbage();
		}
		Modifier = AppliedModifier;
		AssetData->Modify();
		AssetUserDataInterface.GetObject()->Modify();
	}
}

bool UAnimationModifier::IsLatestRevisionApplied(IInterface_AssetUserData* AssetUserDataInterface) const
{
	if (UAnimationModifier* AppliedModifier = GetAppliedModifier(AssetUserDataInterface))
	{
		return AppliedModifier->RevisionGuid == GetLatestRevisionGuid();
	}
	return false;
}

UAnimationModifier* UAnimationModifier::GetLegacyPreviouslyAppliedModifierForModifierOnSkeleton() const
{
	// Read PreviouslyAppliedModifier_DEPRECATED from previous version
	// Check PostLoad() when we moved the deprecated value into this place
	if (UAnimationModifiersAssetUserData* AssetData = Cast<UAnimationModifiersAssetUserData>(GetOuter()))
	{
		if (USkeleton* Skeleton = Cast<USkeleton>(AssetData->GetOuter()))
		{
			if (TObjectPtr<UAnimationModifier>* PtrToModifier = AssetData->AppliedModifiers.Find(this))
			{
				// The new system uses a _placeholder_ object with _exact_ UAnimationAsset class to store RevisionGuid
				// Any object with concrete modifier class must come from PreviouslyAppliedModifier_DEPRECATED
				if (PtrToModifier->GetClass() != UAnimationAsset::StaticClass())
				{
					return PtrToModifier->Get();
				}
			}
		}
	}
	return nullptr;
}

FGuid UAnimationModifier::GetLatestRevisionGuid() const
{
	return GetDefault<UAnimationModifier>(GetClass())->RevisionGuid;
}

void UAnimationModifier::GetAssetRegistryTagsForAppliedModifiersFromSkeleton(FAssetRegistryTagsContext Context)
{
	const UObject* Object = Context.GetObject();
	FString ModifiersList;
	if (UAnimSequence* AnimSequence = const_cast<UAnimSequence*>(Cast<UAnimSequence>(Object)))
	{
		if (USkeleton* Skeleton = AnimSequence->GetSkeleton())
		{
			if (UAnimationModifiersAssetUserData* SkeletonAssetData = Skeleton->GetAssetUserData<UAnimationModifiersAssetUserData>())
			{
				for (const UAnimationModifier* Modifier : SkeletonAssetData->GetAnimationModifierInstances())
				{
					if (const UAnimationModifier* AppliedModifier = Modifier->GetAppliedModifier(AnimSequence))
					{
						// "{Name}={Revision};"
						FString Revision;
						AppliedModifier->RevisionGuid.AppendString(Revision, EGuidFormats::Short);
						ModifiersList += FString::Printf(TEXT("%s%c%s%c"), *Modifier->GetName(), AnimationModifiersAssignment, *Revision, AnimationModifiersDelimiter);
					}
				}
				if (!ModifiersList.IsEmpty())
				{
					Context.AddTag(FAssetRegistryTag(AnimationModifiersTag, ModifiersList, UObject::FAssetRegistryTag::TT_Hidden));
				}
			}
		}
	}
}

//! @brief Mark this modifier on skeleton as reverted (affect CanRevert, IsLatestRevisionApplied)
void UAnimationModifier::RemoveLegacyPreviousAppliedModifierOnSkeleton(USkeleton* Skeleton)
{
	if (UAnimationModifier* AppliedModifier = FindAndRemoveAppliedModifier(Skeleton))
	{
		AppliedModifier->MarkAsGarbage();
	}
}

void UAnimationModifier::PostInitProperties()
{
	Super::PostInitProperties();
	UpdateNativeRevisionGuid();
}

void UAnimationModifier::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);

	/** Back-wards compatibility, assume the current modifier as previously applied */
	if (Ar.CustomVer(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::SerializeAnimModifierState)
	{
		UObject* AssetData = GetOuter();
		UObject* AnimationOrSkeleton = AssetData ? AssetData->GetOuter() : nullptr;
		if (AnimationOrSkeleton)
		{
			SetAppliedModifier(AnimationOrSkeleton, DuplicateObject(this, GetOuter()));
		}
		Modify();
	}
}

void UAnimationModifier::PostLoad()
{
	Super::PostLoad();

	UClass* Class = GetClass();
	UObject* DefaultObject = Class->GetDefaultObject();

	// CDO, set GUID if invalid
	if(DefaultObject == this)
	{
		// Ensure we always have a valid guid
		if (!RevisionGuid.IsValid())
		{
			UpdateRevisionGuid(GetClass());
			MarkPackageDirty();
		}
	}

	if (PreviouslyAppliedModifier_DEPRECATED)
	{
		// The applied guid is now stored at the revision guid of applied modifier
		PreviouslyAppliedModifier_DEPRECATED->RevisionGuid = this->AppliedGuid_DEPRECATED;
		if (UAnimationModifiersAssetUserData* AssetData = Cast<UAnimationModifiersAssetUserData>(GetOuter()))
		{
			if (UAnimSequence* AnimSequence = Cast<UAnimSequence>(AssetData->GetOuter())) 
			{
				UE_LOGF(LogAnimation, Log, "Upgrading Applied Animation Modifier on Sequence %ls.", *AssetData->GetOuter()->GetName());
				UAnimationModifier* AppliedModifier = PreviouslyAppliedModifier_DEPRECATED;
				PreviouslyAppliedModifier_DEPRECATED = nullptr;
				SetAppliedModifier(AnimSequence, AppliedModifier);
			}
			else if (USkeleton* Skeleton = Cast<USkeleton>(AssetData->GetOuter()))
			{
				UE_LOGF(LogAnimation, Warning, "Upgrading Applied Animation Modifier %ls on Skeleton %ls, please reapply the modifier.", *GetName(), *AssetData->GetOuter()->GetName());
				UAnimationModifier* AppliedModifier = PreviouslyAppliedModifier_DEPRECATED;
				PreviouslyAppliedModifier_DEPRECATED = nullptr;
				SetAppliedModifier(Skeleton, AppliedModifier);
			}
			else
			{
				UE_LOGF(LogAnimation, Error, "Upgrading Applied Animation Modifier on Unknown type of asset %ls, Ignored.", *AssetData->GetOuter()->GetName());
				PreviouslyAppliedModifier_DEPRECATED->MarkAsGarbage();
				PreviouslyAppliedModifier_DEPRECATED = nullptr;
			}
		}
		else 
		{
			UE_LOGF(LogAnimation, Error, "Upgrading Applied Animation Modifier on Unknown asset, Ignored.");
			PreviouslyAppliedModifier_DEPRECATED->MarkAsGarbage();
			PreviouslyAppliedModifier_DEPRECATED = nullptr;
		}
		this->Modify();
	}
}

const USkeleton* UAnimationModifier::GetSkeleton()
{
	return CurrentSkeleton;
}

void UAnimationModifier::UpdateRevisionGuid(UClass* ModifierClass)
{
	if (ModifierClass)
	{
		RevisionGuid = FGuid::NewGuid();

	}
}

void UAnimationModifier::UpdateNativeRevisionGuid()
{
	UClass* Class = GetClass();
	// Check if this is the class default object
	if (this == GetDefault<UAnimationModifier>(Class))
	{
		// If so check whether or not the config stored revision matches the natively defined one
		if (StoredNativeRevision != GetNativeClassRevision())
		{
			// If not update the blueprint revision GUID
			UpdateRevisionGuid(Class);
			StoredNativeRevision = GetNativeClassRevision();

			MarkPackageDirty();

			// Save the new native revision to config files
			SaveConfig();
			TryUpdateDefaultConfigFile();
		}
	}
}

void UAnimationModifier::ExecuteOnRevert(UAnimSequence* InAnimSequence)
{
	CurrentAnimSequence = InAnimSequence;
	CurrentSkeleton = InAnimSequence->GetSkeleton();
	OnRevert(InAnimSequence);
	CurrentAnimSequence = nullptr;
	CurrentSkeleton = nullptr;
}

void UAnimationModifier::ExecuteOnApply(UAnimSequence* InAnimSequence)
{	
	CurrentAnimSequence = InAnimSequence;
	CurrentSkeleton = InAnimSequence->GetSkeleton();
	OnApply(InAnimSequence);
	CurrentAnimSequence = nullptr;
	CurrentSkeleton = nullptr;
}

void UAnimationModifier::ApplyToAll(TSubclassOf<UAnimationModifier> ModifierSubClass, bool bForceApply /*= true*/)
{
	if (UClass* ModifierClass = ModifierSubClass.Get())
	{
		// Make sure all packages (in this case UAnimSequences) are loaded to ensure the TObjectIterator has any instances to iterate over
		LoadModifierReferencers(ModifierSubClass);

		for (UAnimationModifiersAssetUserData* AssetUserData : TObjectRange<UAnimationModifiersAssetUserData>{})
		{
			UAnimSequence* AnimSequence = Cast<UAnimSequence>(AssetUserData->GetOuter());
			USkeleton* Skeleton = Cast<USkeleton>(AssetUserData->GetOuter());
			ensure(AnimSequence || Skeleton);

			TUniquePtr<IAnimationDataController::FScopedBracket> ScopedBracket;
			if (AnimSequence)
			{
				IAnimationDataController& Controller = AnimSequence->GetController();
				ScopedBracket = MakeUnique<IAnimationDataController::FScopedBracket>(Controller, LOCTEXT("ApplyingAnimationModifier", "Applying Animation Modifiers"));
			}
			
			for (UAnimationModifier* Modifier : AssetUserData->AnimationModifierInstances)
			{
				if (Modifier->GetClass() == ModifierClass && IsValidChecked(Modifier))
				{
					if (AnimSequence)
					{
						if (bForceApply || !Modifier->IsLatestRevisionApplied(AnimSequence))
						{
							Modifier->ApplyToAnimationSequence(AnimSequence);
						}
					}
					else if (Skeleton) 
					{
						// TODO : Modifier on Skeleton
						// We can apply the modifier to all animation sequences referencing this skeleton
						// Save this behavior change for another CL
						UE_LOGF(LogAnimation, Warning, "Animation Modifier %ls on Skeleton %ls was skipped, please manually apply it from Skeleton", *ModifierClass->GetName(), *Skeleton->GetName());
					}
				}
			}
		}
	}
}

void UAnimationModifier::LoadModifierReferencers(TSubclassOf<UAnimationModifier> ModifierSubClass)
{
	if (UClass* ModifierClass = ModifierSubClass.Get())
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		TArray<FName> PackageDependencies;
		AssetRegistryModule.GetRegistry().GetReferencers(ModifierClass->GetPackage()->GetFName(), PackageDependencies);

		TArray<FString> PackageNames;
		Algo::Transform(PackageDependencies, PackageNames, [](FName Name) { return Name.ToString(); });
		TArray<UPackage*> Packages = AssetViewUtils::LoadPackages(PackageNames);
	}
}

int32 UAnimationModifier::GetNativeClassRevision() const
{
	// Overriden in derrived classes to perform native revisioning
	return 0;
}

const UAnimSequence* UAnimationModifier::GetAnimationSequence()
{
	return CurrentAnimSequence;
}

namespace UE::Anim
{
	TArray<FApplyModifiersScope::ESuppressionMode, TInlineAllocator<4>> FApplyModifiersScope::ScopeModeStack;
	TMap<FObjectKey, EAppReturnType::Type> FApplyModifiersScope::WarningResponse;
	TSet<FObjectKey> FApplyModifiersScope::ErrorResponse;

	FApplyModifiersScope::ESuppressionMode FApplyModifiersScope::CurrentMode()
	{
		// Force Dialog is the legacy behavior when no scope is open
		return ScopeModeStack.IsEmpty() ? ForceDialog : ScopeModeStack.Top();
	}

	FApplyModifiersScope::ESuppressionMode FApplyModifiersScope::Open(ESuppressionMode Mode)
	{
		if (Mode == NoChange)
		{
			// When no scope was open, we should be transit to ShowDialog mode (not ForceDialog mode)
			Mode = ScopeModeStack.IsEmpty() ? ShowDialog : ScopeModeStack.Top();
		}
		if (ScopeModeStack.IsEmpty())
		{
			// Clear the response at the out most scope open
			WarningResponse.Empty();
			ErrorResponse.Empty();
		}
		ScopeModeStack.Push(Mode);
		return Mode;
	}

	void FApplyModifiersScope::Close()
	{
		if (!ensureMsgf(!ScopeModeStack.IsEmpty(), TEXT("Unpaired FApplyModifiersScope::Open/Close call.")))
		{
			return;
		}

		ScopeModeStack.Pop();
		if (ScopeModeStack.IsEmpty())
		{
			// Clear the response at the out most scope close
			WarningResponse.Empty();
			ErrorResponse.Empty();
		}
	}

	void FApplyModifiersScope::HandleError(const UAnimationModifier* Modifier, const FText& Message, const FText& Title)
	{
		ESuppressionMode Mode = CurrentMode();
		if (Mode > SuppressWarningAndError && Mode < RevertAtWarning)
		{
			// Show the dialog if this error was not shown before, or when forced
			if (Mode == ForceDialog || !ErrorResponse.Contains(Modifier->GetClass()))
			{
				FMessageDialog::Open(EAppMsgType::Ok, Message, Title);
			}
		}
		// Record the error to avoid same error dialog
		ErrorResponse.Add(Modifier->GetClass());
	}

	bool FApplyModifiersScope::HandleWarning(const UAnimationModifier* Modifier, const FText& Message, const FText& Title)
	{
		auto IsYesAllNoAll = [](EAppReturnType::Type Value) { return Value == EAppReturnType::YesAll || Value == EAppReturnType::NoAll; };
		auto PtrValueOr = [](auto* Ptr, auto Default) { return Ptr ? *Ptr : Default; };

		EAppReturnType::Type Response = EAppReturnType::No;
		switch (ESuppressionMode Mode = CurrentMode())
		{
		case SuppressWarningAndError:
		case SuppressWarning:
			Response = EAppReturnType::Yes;
			break;
		case RevertAtWarning:
			Response = EAppReturnType::No;
			break;
		case ForceDialog:
			Response = EAppReturnType::Retry;
			break;
		case ShowDialog:
		default:
			Response = PtrValueOr(WarningResponse.Find(Modifier->GetClass()), EAppReturnType::Retry);
			break;
		}

		// Don't show the dialog if the previous response is YesToAll or NoToAll
		if (Response == EAppReturnType::Retry)
		{
			Response = FMessageDialog::Open(EAppMsgType::YesNoYesAllNoAll, Message, Title);
			// Record the YesAll NoAll response to avoid same the warning dialog
			if (IsYesAllNoAll(Response))
			{
				WarningResponse.Add(Modifier->GetClass(), Response);
			}
		}

		return (Response == EAppReturnType::Yes) || (Response == EAppReturnType::YesAll);
	}
} // namespace UE::Anim

#undef LOCTEXT_NAMESPACE
