// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationModifierDetailCustomization.h"

#include "AnimationModifier.h"
#include "Containers/Array.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "ScopedTransaction.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "FAnimationModifierDetailCustomization"

void FAnimationModifierDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	ModifierInstance = nullptr;
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);
	ensure(Objects.Num() == 1); // This is in here to ensure we are only showing the modifier details in the blueprint editor

	for (TWeakObjectPtr<UObject> Object : Objects)
	{ 
		if (Object->IsA<UAnimationModifier>())
		{
			ModifierInstance = Cast<UAnimationModifier>(Object.Get());
		}
	}
	
	// If we have found a valid modifier instance add a revision bump button to the details panel
	if (ModifierInstance)
	{
		IDetailCategoryBuilder& RevisionCategory = DetailBuilder.EditCategory("Modifier Instances");
		FDetailWidgetRow& UpdateRevisionRow = RevisionCategory.AddCustomRow(LOCTEXT("ModifierActionsLabel", "Modifier Actions"))
		.WholeRowWidget
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f)
			[
			   SNew(SButton)
			   .OnClicked(this, &FAnimationModifierDetailCustomization::OnApplyButtonClicked, true)
			   .ToolTipText(LOCTEXT("ApplyToolTip", "Applies any instanced modifiers of this class to their owning Animation Sequences."))
			   .Text(LOCTEXT("ApplyText", "Apply to All"))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f)
			[
				SNew(SButton)
				.OnClicked(this, &FAnimationModifierDetailCustomization::OnApplyButtonClicked, false)
				.ToolTipText(LOCTEXT("ApplyOutOfDataToolTip", "Applies any instanced modifiers, if they are out-of-date, of this class to their owning Animation Sequences."))
				.Text(LOCTEXT("ApplyToOutOfDateText", "Apply to All out-of-date"))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f)
			[
				SAssignNew(UpdateRevisionButton, SButton)
				.OnClicked(this, &FAnimationModifierDetailCustomization::OnUpdateRevisionButtonClicked)
				.ToolTipText(LOCTEXT("UpdateRevisionToolTip", "Updates the stored revision GUID on all instances of this Modifier class, marking them out-of-date."))
				.Text(LOCTEXT("UpdateRevisionText", "Update Revision"))
			]           
		];
	}
}

FReply FAnimationModifierDetailCustomization::OnUpdateRevisionButtonClicked()
{
	if (ModifierInstance)
	{
		ModifierInstance->UpdateRevisionGuid(ModifierInstance->GetClass());
	}
	return FReply::Handled();
}

FReply FAnimationModifierDetailCustomization::OnApplyButtonClicked(bool bForceApply)
{
	if (ModifierInstance)
	{		
		const FScopedTransaction Transaction(LOCTEXT("UndoAction_ApplyModifiers", "Applying Animation Modifier to Animation Sequence(s)"));		
		UAnimationModifier::ApplyToAll(ModifierInstance->GetClass(), bForceApply);
	}
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE //"FAnimationModifierDetailCustomization"
