// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Templates/SharedPointer.h"

class IDetailLayoutBuilder;
class SButton;
class UAnimationModifier;

/** Animation modifier detail customization */
class FAnimationModifierDetailCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance() { return MakeShareable(new FAnimationModifierDetailCustomization());  }

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
protected:
	FReply OnUpdateRevisionButtonClicked();
	FReply OnApplyButtonClicked(bool bForceApply);
protected:
	TSharedPtr<SButton> UpdateRevisionButton;
	UAnimationModifier* ModifierInstance;
};
