// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClassViewerModule.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "ClassViewerFilter.h"
#include "AnimationModifier.h"
#include "AnimationModifiersAssetUserData.h"
#include "AnimationModifier.h"

class FAnimationModifierHelpers
{
public:
	/** ClassViewerFilter for Animation Modifier classes */
	class FModifierClassFilter : public IClassViewerFilter
	{
	public:
		bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			return InClass->IsChildOf(UAnimationModifier::StaticClass());
		}

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			return InClass->IsChildOf(UAnimationModifier::StaticClass());
		}
	};

	static TSharedRef<SWidget> GetModifierPicker(const FOnClassPicked& OnClassPicked)
	{
		FClassViewerInitializationOptions Options;
		Options.bShowUnloadedBlueprints = true;
		Options.bShowNoneOption = false;
		TSharedRef<FModifierClassFilter> ClassFilter = MakeShared<FModifierClassFilter>();
		Options.ClassFilters.Add(ClassFilter);

		return SNew(SBox)
		.WidthOverride(280)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.MaxHeight(500)
			[
				FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(Options, OnClassPicked)
			]
		];
	}

	/** Creates a new Modifier instance to store with the current asset */
	static UAnimationModifier* CreateModifierInstance(UObject* Outer, UClass* InClass, UObject* Template = nullptr)
	{
		checkf(Outer, TEXT("Invalid outer value for modifier instantiation"));
		UAnimationModifier* ProcessorInstance = NewObject<UAnimationModifier>(Outer, InClass, NAME_None, RF_NoFlags, Template);
		checkf(ProcessorInstance, TEXT("Unable to instantiate modifier class"));
		ProcessorInstance->SetFlags(RF_Transactional);
		return ProcessorInstance;
	}

	static UAnimationModifiersAssetUserData* RetrieveOrCreateModifierUserData(TScriptInterface<IInterface_AssetUserData> AssetUserDataInterface)
	{
		UAnimationModifiersAssetUserData* AssetUserData = AssetUserDataInterface->GetAssetUserData<UAnimationModifiersAssetUserData>();
		if (!AssetUserData)
		{
			AssetUserData = NewObject<UAnimationModifiersAssetUserData>(AssetUserDataInterface.GetObject(), UAnimationModifiersAssetUserData::StaticClass());
			checkf(AssetUserData, TEXT("Unable to instantiate AssetUserData class"));
			AssetUserData->SetFlags(RF_Transactional);
			AssetUserDataInterface->AddAssetUserData(AssetUserData);
		}

		return AssetUserData;
	}

	//! @brief Enumerate the List[ModifierName,RevisionGuid] formed tag
	//! @param[in] Tag The tag to parse
	//! @param[in] Callback (FStringView, FGuid) -> bool, the function to apply to each modifier=revision pair, return false to _break_ from enumeration
	template<typename Func>
	static void EnumerateAnimationModifierTags(FStringView Tag, Func&& Callback)
	{
		static_assert(std::is_invocable_r_v<bool, Func, FStringView, FGuid>, "Func must be invocable (FStringView, FGuid) -> bool");
		int Begin = 0;
		int Mid = -1;
		int End = -1;
		for (size_t I = 0; I <= Tag.Len(); I++)
		{
			if (I == Tag.Len() || Tag[I] == UAnimationModifier::AnimationModifiersDelimiter)
			{
				if (I <= Begin)
				{
					continue; // Skip empty pair
				}
				End = I;
				ensure(Mid >= Begin && End > Mid);
				FStringView Name = Tag.SubStr(Begin, Mid - Begin);
				FStringView Revision = Tag.SubStr(Mid + 1, End - Mid - 1);
				FGuid RevisionGuid;
				FGuid::Parse(FString{Revision}, RevisionGuid);

				bool Break = !Invoke(Callback, Name, RevisionGuid);
				if (Break)
				{
					break;
				}
				Begin = End + 1;
			}
			else if (Tag[I] == UAnimationModifier::AnimationModifiersAssignment)
			{
				Mid = I;
			}
		}
	}
};
