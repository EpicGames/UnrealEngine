// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SubclassOf.h"
#include "Engine/DeveloperSettings.h"
#include "AnimationModifier.h"
#include "AnimationModifierSettings.generated.h"

UCLASS(config=Engine, meta=(DisplayName="Animation Modifiers"))
class UAnimationModifierSettings : public UDeveloperSettings
{
public:
	GENERATED_BODY()
	
	/** Set of Animation Modifiers to be added whenever a new Animation Sequence is imported */
	UPROPERTY(config, EditAnywhere, Category = Modifiers)
	TArray<TSubclassOf<UAnimationModifier>> DefaultAnimationModifiers;

	/** Whether or not to apply animation modifiers post (re)import */
	UPROPERTY(config, EditAnywhere, Category = Modifiers)
	bool bApplyAnimationModifiersOnImport = false;
};