// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationModifiersAssetUserData.h"
#include "AnimationModifierHelpers.h"
#include "UObject/UObjectThreadContext.h"

#include "HAL/PlatformCrt.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/AssertionMacros.h"
#include "UObject/ObjectPtr.h"
#include "Animation/AnimSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationModifiersAssetUserData)

UAnimationModifiersAssetUserData::UAnimationModifiersAssetUserData(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	if (!HasAnyFlags(RF_ClassDefaultObject) && !HasAnyFlags(RF_Transactional))
	{
		SetFlags(RF_Transactional);
	}
}

void UAnimationModifiersAssetUserData::AddAnimationModifier(UAnimationModifier* Instance)
{
	AnimationModifierInstances.Add(Instance);
}

void UAnimationModifiersAssetUserData::RemoveAnimationModifierInstance(UAnimationModifier* Instance)
{
	checkf(AnimationModifierInstances.Contains(Instance), TEXT("Instance suppose to be removed is not found"));
	AnimationModifierInstances.Remove(Instance);
	AppliedModifiers.Remove(Instance);
}

const TArray<UAnimationModifier*>& UAnimationModifiersAssetUserData::GetAnimationModifierInstances() const
{
	return AnimationModifierInstances;
}

bool UAnimationModifiersAssetUserData::AddAnimationModifierOfClass(UAnimSequenceBase* AnimationSequenceBase, TSubclassOf<UAnimationModifier> InModifierClass)
{
	if (AnimationSequenceBase != nullptr)
	{
		UClass* ModifierClassToAdd = InModifierClass.Get();
		if (ModifierClassToAdd != nullptr)
		{
			if (UAnimationModifiersAssetUserData* AssetUserData = FAnimationModifierHelpers::RetrieveOrCreateModifierUserData(AnimationSequenceBase))
			{
				if(UAnimationModifier* ModifierInstance = FAnimationModifierHelpers::CreateModifierInstance(AssetUserData, ModifierClassToAdd))
				{					
					AssetUserData->AddAnimationModifier(ModifierInstance);
					return true;
				}
				else
				{
					UE_LOGF(LogAnimation, Warning, "Failed to instantiate Animation Modifier class supplied for AddAnimationModifierOfClass");
				}
			}
			else
			{
				UE_LOGF(LogAnimation, Warning, "Invalid Animation Modifier class supplied for AddAnimationModifierOfClass");
			}
			
		}
		else
		{
			UE_LOGF(LogAnimation, Warning, "Invalid Animation Modifier class supplied for AddAnimationModifierOfClass");
		}
	}
	else
	{
		UE_LOGF(LogAnimation, Warning, "Invalid Animation Sequence supplied for AddAnimationModifierOfClass");
	}

	return false;
}

void UAnimationModifiersAssetUserData::ChangeAnimationModifierIndex(UAnimationModifier* Instance, int32 Direction)
{
	checkf(AnimationModifierInstances.Contains(Instance), TEXT("Instance suppose to be moved is not found"));
	const int32 CurrentIndex = AnimationModifierInstances.IndexOfByKey(Instance);
	const int32 NewIndex = FMath::Clamp(CurrentIndex + Direction, 0, AnimationModifierInstances.Num() - 1);
	if (CurrentIndex != NewIndex)
	{
		AnimationModifierInstances.Swap(CurrentIndex, NewIndex);
	}
}

void UAnimationModifiersAssetUserData::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	RemoveInvalidModifiers();
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#if WITH_EDITOR
void UAnimationModifiersAssetUserData::PostEditChangeOwner(const FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeOwner(PropertyChangedEvent);

	// We cant call blueprint implemented functions while routing post load
	if (FUObjectThreadContext::Get().IsRoutingPostLoad)
	{
		return;
	}

	// Skip interactive changes
	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::Interactive)
	{
		return;
	}
	
	// Only react to PostEditChange if not currently applying modifiers
	if(!UE::Anim::FApplyModifiersScope::IsScopePending())
	{		
		if (UAnimSequence* ModifiedAnimSequence = Cast<UAnimSequence>(GetOuter()))
		{
			UE::Anim::FApplyModifiersScope Scope;
			for (const UAnimationModifier* Modifier : AnimationModifierInstances)
			{
				if (!Modifier->bReapplyPostOwnerChange)
				{
					continue;
				}
				Modifier->ApplyToAnimationSequence(ModifiedAnimSequence);
			}

			// Find outer USkeleton, and apply any skeleton-level modifiers
			if (USkeleton* OuterSkeleton = ModifiedAnimSequence->GetSkeleton())
			{
				if (UAnimationModifiersAssetUserData* SkeletonAssetUserData = OuterSkeleton->GetAssetUserData<UAnimationModifiersAssetUserData>())
				{
				    for (const UAnimationModifier* Modifier : SkeletonAssetUserData->GetAnimationModifierInstances())
				    {
					    if (!Modifier->bReapplyPostOwnerChange)
					    {
						    continue;
					    }
    
					    Modifier->ApplyToAnimationSequence(ModifiedAnimSequence);
				    }
				}
			}
		}
	}
}
#endif // WITH_EDITOR

void UAnimationModifiersAssetUserData::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
}

void UAnimationModifiersAssetUserData::PostLoad()
{
	Super::PostLoad();
	if (!HasAnyFlags(RF_ClassDefaultObject) && !HasAnyFlags(RF_Transactional))
	{		
		SetFlags(RF_Transactional);
	}
	RemoveInvalidModifiers();
}

void UAnimationModifiersAssetUserData::RemoveInvalidModifiers()
{
	// This will catch force-deleted blueprints to be removed from our stored array
	AnimationModifierInstances.RemoveAll([](UAnimationModifier* Modifier) { return Modifier == nullptr; });

	// The AssetUserData on this animation's skeleton
	UAnimationModifiersAssetUserData* SkeletonAssetUserData = nullptr;
	if (const UObject* OwnerAsset = GetOuter())
	{
		if (const UAnimSequence* Sequence = Cast<UAnimSequence>(OwnerAsset->GetOuter()))
		{
			if (USkeleton* Skeleton = Sequence->GetSkeleton())
			{
				SkeletonAssetUserData = Skeleton->GetAssetUserData< UAnimationModifiersAssetUserData>();
			}
		}
	}

	auto IsAppliedModifierValid = [this, SkeletonAssetUserData](const decltype(AppliedModifiers)::ElementType& Pair)
	{
		// The key (template modifier) from both animation sequence or skeleton should be loaded before this object
		if (const UObject* TemplateModifier = Pair.Key.ResolveObject())
		{
			if (const UObject* AssetData = TemplateModifier->GetOuter())
			{
				// If the TemplateModifier is not owned by us or the skeleton
				// This asset user data must been re-parented
				// Remove those stale applied-modifier information
				return AssetData == this || AssetData == SkeletonAssetUserData;
			}
		}
		return false;
	};
	// Remove all applied modifier instances of deleted modifier
	AppliedModifiers = AppliedModifiers.FilterByPredicate([](const decltype(AppliedModifiers)::ElementType& Pair) { return Pair.Key.ResolveObject() != nullptr; });
}
