// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationModifiersModule.h"

#include "Algo/Copy.h"
#include "Animation/AnimSequence.h"
#include "AnimationModifier.h"
#include "AnimationModifierDetailCustomization.h"
#include "AnimationModifierHelpers.h"
#include "AnimationModifierSettings.h"
#include "AnimationModifiersAssetUserData.h"
#include "AnimationModifiersTabSummoner.h"
#include "Delegates/Delegate.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Interfaces/IMainFrameModule.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "Internationalization/Internationalization.h"
#include "Math/Vector2D.h"
#include "Misc/CoreDelegates.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "SAnimationModifierContentBrowserWindow.h"
#include "ScopedTransaction.h"
#include "Subsystems/ImportSubsystem.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"
#include "WorkflowOrientedApp/ApplicationMode.h"

#include "AnimationModifierSettings.h"

#include "AnimationModifiersAssetUserData.h"
#include "AssetToolsModule.h"
#include "AssetViewUtils.h"
#include "ContentBrowserMenuContexts.h"
#include "ToolMenuDelegates.h"
#include "ToolMenus.h"
#include "AssetRegistry/AssetRegistryModule.h"

class UFactory;

#define LOCTEXT_NAMESPACE "AnimationModifiersModule"

IMPLEMENT_MODULE(FAnimationModifiersModule, AnimationModifiers);

const FName FAnimationModifiersTabSummoner::AnimationModifiersName = TEXT("AnimationModifiers");

void FAnimationModifiersModule::StartupModule()
{
	// Register class/struct customizations
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyEditorModule.RegisterCustomClassLayout("AnimationModifier", FOnGetDetailCustomizationInstance::CreateStatic(&FAnimationModifierDetailCustomization::MakeInstance));
	
	// Add application mode extender
	Extender = FWorkflowApplicationModeExtender::CreateRaw(this, &FAnimationModifiersModule::ExtendApplicationMode);
	FWorkflowCentricApplication::GetModeExtenderList().Add(Extender);
	
	// Register delegates during PostEngineInit as this module is part of preload phase and GEditor is not valid yet
	DelegateHandle = FCoreDelegates::GetOnPostEngineInit().AddLambda([this]()
	{
		if (GEditor)
		{
			IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
			
			AssetAction = MakeShared<FAssetTypeActions_AnimationModifier>();
			AssetTools.RegisterAssetTypeActions(AssetAction.ToSharedRef());
			
			GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostImport.AddRaw(this, &FAnimationModifiersModule::OnAssetPostImport);
			GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetReimport.AddRaw(this, &FAnimationModifiersModule::OnAssetPostReimport);
			RegisterMenus();

			const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
			AssetRegistryModule.Get().OnInMemoryAssetCreated().AddRaw(this, &FAnimationModifiersModule::OnInMemoryAssetCreated);
		}
	});

	// Register extra asset registry tags for UAnimSequence
	OnGetExtraObjectTagsHandle = UObject::FAssetRegistryTag::OnGetExtraObjectTagsWithContext.AddStatic(&UAnimationModifier::GetAssetRegistryTagsForAppliedModifiersFromSkeleton);
}

void FAnimationModifiersModule::ShutdownModule()
{
	// Make sure we unregister the class layout 
	FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
	if (PropertyEditorModule)
	{
		PropertyEditorModule->UnregisterCustomClassLayout("AnimationModifier");
	}
	
	UToolMenus::UnregisterOwner(this);
	FCoreDelegates::GetOnPostEngineInit().Remove(DelegateHandle);

	UObject::FAssetRegistryTag::OnGetExtraObjectTagsWithContext.Remove(OnGetExtraObjectTagsHandle);
	
	// Remove extender delegate
	FWorkflowCentricApplication::GetModeExtenderList().RemoveAll([this](FWorkflowApplicationModeExtender& StoredExtender) { return StoredExtender.GetHandle() == Extender.GetHandle(); });

	// During shutdown clean up all factories from any modes which are still active/alive
	for (TWeakPtr<FApplicationMode> WeakMode : RegisteredApplicationModes)
	{
		if (WeakMode.IsValid())
		{
			TSharedPtr<FApplicationMode> Mode = WeakMode.Pin();
			Mode->RemoveTabFactory(FAnimationModifiersTabSummoner::AnimationModifiersName);
		}
	}

	if (FModuleManager::Get().IsModuleLoaded(TEXT("AssetTools")))
	{
		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		AssetTools.UnregisterAssetTypeActions(AssetAction.ToSharedRef());
	}

	if (FModuleManager::Get().IsModuleLoaded(TEXT("AssetRegistry")))
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		AssetRegistryModule.Get().OnInMemoryAssetCreated().RemoveAll(this);
	}
	
	RegisteredApplicationModes.Empty();

	if (GEditor)
	{
		GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostImport.RemoveAll(this);
		GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetReimport.RemoveAll(this);
	}
}

TSharedRef<FApplicationMode> FAnimationModifiersModule::ExtendApplicationMode(const FName ModeName, TSharedRef<FApplicationMode> InMode)
{
	// For skeleton and animation editor modes add our custom tab factory to it
	if (ModeName == TEXT("SkeletonEditorMode") || ModeName == TEXT("AnimationEditorMode"))
	{
		InMode->AddTabFactory(FCreateWorkflowTabFactory::CreateStatic(&FAnimationModifiersTabSummoner::CreateFactory));
		RegisteredApplicationModes.Add(InMode);
	}
	
	return InMode;
}

void FAnimationModifiersModule::RegisterMenus()
{
	FToolMenuOwnerScoped OwnerScoped(this);
	
	UToolMenus* ToolMenus = UToolMenus::Get();
	UToolMenu* Menu = ToolMenus->ExtendMenu("ContentBrowser.AssetContextMenu.AnimSequence");
	if (!Menu)
	{
		return;
	}

	FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");

	Section.AddDynamicEntry("AnimModifierActions",
		FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
		{
			const UContentBrowserAssetContextMenuContext* Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>();
			if (!Context)
			{
				return;
			}

			TArray<FAssetData> AnimSequenceAssets;
			Algo::CopyIf(Context->SelectedAssets, AnimSequenceAssets, [](const FAssetData& AssetData)
			{
				return AssetData.AssetClassPath == UAnimSequence::StaticClass()->GetClassPathName();
			});
			
			auto GetAnimSequences = [AnimSequenceAssets](TArray<UAnimSequence*>& OutSequences)
			{
				TArray<UObject*> Objects;
				AssetViewUtils::LoadAssetsIfNeeded(AnimSequenceAssets, Objects, AssetViewUtils::FLoadAssetsSettings{});
			
				Algo::TransformIf(Objects, OutSequences, 
				[](const UObject* Object)
				{
					return Cast<UAnimSequence>(Object) != nullptr;
				},
				[](UObject* Object)
				{
					return Cast<UAnimSequence>(Object);
				});
			};
				
			const FNewMenuDelegate MenuDelegate = FNewMenuDelegate::CreateLambda([GetAnimSequences, this](FMenuBuilder& MenuBuilder)
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("AnimSequence_AddAnimationModifier", "Add Modifiers"),
					LOCTEXT("AnimSequence_AddAnimationModifierTooltip", "Add new animation modifier(s)."),
				   FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.AnimationModifier"),
					FUIAction(FExecuteAction::CreateLambda([GetAnimSequences, this]()
					{					
						TArray<UAnimSequence*> AnimSequences;
						GetAnimSequences(AnimSequences);

						ShowAddAnimationModifierWindow(AnimSequences);
					}))
				);
			
				MenuBuilder.AddMenuEntry(
					LOCTEXT("AnimSequence_ApplyAnimationModifier", "Apply Modifiers"),
					LOCTEXT("AnimSequence_ApplyAnimationModifierTooltip", "Applies all contained animation modifier(s)."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.AnimationModifier"),
					FUIAction(FExecuteAction::CreateLambda([GetAnimSequences, this]()
					{					
						TArray<UAnimSequence*> AnimSequences;
						GetAnimSequences(AnimSequences);
						
						ApplyAnimationModifiers(AnimSequences);
					}))
				);

				MenuBuilder.AddMenuEntry(
					LOCTEXT("AnimSequence_ApplyOutOfDataAnimationModifier", "Apply out-of-date Modifiers"),
					LOCTEXT("AnimSequence_ApplyOutOfDataAnimationModifierTooltip", "Applies all contained animation modifier(s), if they are out of date."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.AnimationModifier"),
					FUIAction(FExecuteAction::CreateLambda([GetAnimSequences, this]()
					{					
						TArray<UAnimSequence*> AnimSequences;
						GetAnimSequences(AnimSequences);
						
						ApplyAnimationModifiers(AnimSequences, false);
					}))
				);

				MenuBuilder.AddMenuEntry(
					LOCTEXT("AnimSequence_RemoveAnimationModifier", "Remove Modifiers"),
					LOCTEXT("AnimSequence_RemoveAnimationModifierTooltip", "Remove animation modifier(s)"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.AnimationModifier"),
					FUIAction(FExecuteAction::CreateLambda([GetAnimSequences, this]()
					{
						TArray<UAnimSequence*> AnimSequences;
						GetAnimSequences(AnimSequences);

						ShowRemoveAnimationModifierWindow(AnimSequences);
					})));
			});

			IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
			if (AssetTools.IsAssetClassSupported(UAnimationModifier::StaticClass()))
			{
				InSection.AddSubMenu("AnimSequence_AnimationModifiers", LOCTEXT("AnimSequence_AnimationModifiers", "Animation Modifier(s)"),
				LOCTEXT("AnimSequence_AnimationModifiersTooltip", "Animation Modifier actions"),
					FNewToolMenuChoice(MenuDelegate),
					false,
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.AnimationModifier")
				);
			}
		})
	);
}

void FAnimationModifiersModule::OnAssetPostImport(UFactory* ImportFactory, UObject* ImportedObject)
{
	// Check whether or not the imported asset is a AnimSequence
	if (UAnimSequence* AnimationSequence = Cast<UAnimSequence>(ImportedObject))
	{
		// Check whether or not there are any default modifiers which should be added to the new sequence
		const TArray<TSubclassOf<UAnimationModifier>>& DefaultModifiers = GetDefault<UAnimationModifierSettings>()->DefaultAnimationModifiers;
		if (DefaultModifiers.Num())
		{
			UAnimationModifiersAssetUserData* AssetUserData = FAnimationModifierHelpers::RetrieveOrCreateModifierUserData(AnimationSequence);			
			for (TSubclassOf<UAnimationModifier> ModifierClass : DefaultModifiers)
			{
				if (ModifierClass.Get())
				{
					UObject* Outer = AssetUserData;
					UAnimationModifier* Processor = FAnimationModifierHelpers::CreateModifierInstance(Outer, *ModifierClass);
					AssetUserData->Modify();
					AssetUserData->AddAnimationModifier(Processor);
				}
			}

			if (GetDefault<UAnimationModifierSettings>()->bApplyAnimationModifiersOnImport)
			{
				ApplyAnimationModifiers({AnimationSequence});
			}
		}
	}
}

void FAnimationModifiersModule::OnAssetPostReimport(UObject* ReimportedObject)
{
	// Check whether or not the reimported asset is a AnimSequence
	if (UAnimSequence* AnimationSequence = Cast<UAnimSequence>(ReimportedObject))
	{
		// Check whether or not any contained modifiers should be applied 
		if (GetDefault<UAnimationModifierSettings>()->bApplyAnimationModifiersOnImport)
		{			
			ApplyAnimationModifiers({AnimationSequence});
		}
	}
}

void FAnimationModifiersModule::OnInMemoryAssetCreated(UObject* Object)
{
	if (Object->GetClass() == UAnimSequence::StaticClass())
	{
		FAnimationModifierHelpers::RetrieveOrCreateModifierUserData(Cast<UAnimSequence>(Object));
	}
}

void FAnimationModifiersModule::ShowAddAnimationModifierWindow(const TArray<UAnimSequence*>& InSequences)
{
	TSharedPtr<SAnimationModifierContentBrowserWindow> WindowContent;

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("WindowTitle", "Add Animation Modifier(s)"))
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(500, 500));

	Window->SetContent
	(
		SAssignNew(WindowContent, SAnimationModifierContentBrowserWindow)
		.WidgetWindow(Window)
		.AnimSequences(InSequences)
	);

	TSharedPtr<SWindow> ParentWindow;

	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		ParentWindow = MainFrame.GetParentWindow();
	}

	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);
}

void FAnimationModifiersModule::ShowRemoveAnimationModifierWindow(const TArray<UAnimSequence*>& InSequences)
{
	TSharedPtr<SRemoveAnimationModifierContentBrowserWindow> WindowContent;

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("RemoveModifiersWindowTitle", "Remove Animation Modifier(s)"))
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(500, 500));

	Window->SetContent
	(
		SAssignNew(WindowContent, SRemoveAnimationModifierContentBrowserWindow)
		.WidgetWindow(Window)
		.AnimSequences(InSequences)
	);

	TSharedPtr<SWindow> ParentWindow;

	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		ParentWindow = MainFrame.GetParentWindow();
	}

	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);
}

void FAnimationModifiersModule::ApplyAnimationModifiers(const TArray<UAnimSequence*>& InSequences, bool bForceApply /*= true*/)
{
	const FScopedTransaction Transaction(LOCTEXT("UndoAction_ApplyModifiers", "Applying Animation Modifier(s) to Animation Sequence(s)"));
	
	// Iterate over each Animation Sequence and all of its contained modifiers, applying each one
	UE::Anim::FApplyModifiersScope Scope;
	TArray<UAnimationModifiersAssetUserData*> AssetUserData;
	FScopedSlowTask SlowTaskSequences(InSequences.Num(), LOCTEXT("SlowTaskApplyingModifier_Sequence_Start", "Applying Animation Modifier(s) to Animation Sequence(s)..."));
	SlowTaskSequences.MakeDialog();
	for (UAnimSequence* AnimationSequence : InSequences)
	{
		FText AnimSequenceName = FText::FromString(IsValid(AnimationSequence) ? AnimationSequence->GetName() : TEXT(""));
		SlowTaskSequences.EnterProgressFrame(1.0f, FText::Format(LOCTEXT("SlowTaskApplyingModifier_Sequence_Update", "Applying Animation Modifier(s) to Animation Sequence {0}..."), AnimSequenceName));
		if (AnimationSequence)
		{
			UAnimationModifiersAssetUserData* UserData = AnimationSequence->GetAssetUserData<UAnimationModifiersAssetUserData>();
			if (UserData)
			{
				const TArray<UAnimationModifier*>& ModifierInstances = UserData->GetAnimationModifierInstances();
				FScopedSlowTask SlowTaskModifiers(ModifierInstances.Num(), LOCTEXT("SlowTaskApplyingModifier_Modifier_Start", "Applying Animation Modifiers..."));
				SlowTaskModifiers.MakeDialog();
				for (UAnimationModifier* Modifier : ModifierInstances)
				{
					SlowTaskModifiers.EnterProgressFrame(1.0f, FText::Format(LOCTEXT("SlowTaskApplyingModifier_Modifier_Update", "Applying Animation Modifier {0}..."), Modifier->GetClass()->GetDisplayNameText()));
					if (bForceApply || !Modifier->IsLatestRevisionApplied(AnimationSequence))
					{
						Modifier->ApplyToAnimationSequence(AnimationSequence);
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE // "AnimationModifiersModule"
