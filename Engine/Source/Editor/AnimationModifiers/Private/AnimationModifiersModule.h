// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Containers/Array.h"
#include "IAnimationModifiersModule.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"
#include "AnimationModifier.h"

#include "WorkflowOrientedApp/ApplicationMode.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "AssetTypeActions_Base.h"

class UAnimSequence;
class UFactory;
class UObject;

class FAssetTypeActions_AnimationModifier : public FAssetTypeActions_Base
{
public:
	// IAssetTypeActions Implementation
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_AnimationModifer", "Animation Modifier"); }
	virtual FColor GetTypeColor() const override { return FColor(50,162,232); }
	virtual UClass* GetSupportedClass() const override { return UAnimationModifier::StaticClass(); }
	virtual bool HasActions(const TArray<UObject*>& InObjects) const override { return false; }
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Animation; }
};

/** Animation modifiers module, handles injecting of the AnimationModifiersTab into animation and skeleton editor modes */
class FAnimationModifiersModule : public IAnimationModifiersModule
{
public:
	/** Called right after the module DLL has been loaded and the module object has been created */
	virtual void StartupModule() override;

	/** Called before the module is unloaded, right before the module object is destroyed */
	virtual void ShutdownModule() override;

	/** Begin IAnimationModifiersModule overrides */
	virtual void ShowAddAnimationModifierWindow(const TArray<UAnimSequence*>& InSequences) override;
	virtual void ShowRemoveAnimationModifierWindow(const TArray<UAnimSequence*>& InSequences) override;
	virtual void ApplyAnimationModifiers(const TArray<UAnimSequence*>& InSequences, bool bForceApply = true) override;
	/** End IAnimationModifiersModule overrides */

protected:
	/** Callback for extending an application mode */
	TSharedRef<FApplicationMode> ExtendApplicationMode(const FName ModeName, TSharedRef<FApplicationMode> InMode);

	void RegisterMenus();

	/** Weak list of application modes for which a tab factory was registered */
	TArray<TWeakPtr<FApplicationMode>> RegisteredApplicationModes;

	FWorkflowApplicationModeExtender Extender;

	FDelegateHandle OnGetExtraObjectTagsHandle;

	/** Callbacks used to add and apply default animation modifier classes */
	void OnAssetPostImport(UFactory* ImportFactory, UObject* ImportedObject);
	void OnAssetPostReimport(UObject* ReimportedObject);
	void OnInMemoryAssetCreated(UObject* Object);
	
	TSharedPtr<FAssetTypeActions_AnimationModifier> AssetAction;
	FDelegateHandle DelegateHandle;
};
