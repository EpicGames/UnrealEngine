// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/AppStyle.h"
#include "SAnimationModifiersTab.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

#define LOCTEXT_NAMESPACE "AnimationModifiersModule"

/** Tab summoner which creates the Animation modifiers tab inside of the animation and skeleton editor */
struct FAnimationModifiersTabSummoner : public FWorkflowTabFactory
{
public:
	/** Tab ID name */
	static const FName AnimationModifiersName;

	FAnimationModifiersTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp)
		: FWorkflowTabFactory(AnimationModifiersName, InHostingApp)
	{
		TabLabel = LOCTEXT("AnimationModifiersTabLabel", "Animation Data Modifiers");
		TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.AnimationModifier");
	}

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override
	{
		return SNew(SAnimationModifiersTab).InHostingApp(HostingApp);
	}

	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override
	{
		return LOCTEXT("AnimationModifiersTabToolTip", "Tab for Managing Animation Modifier Blueprints");
	}

	static TSharedPtr<FWorkflowTabFactory> CreateFactory(TSharedPtr<FAssetEditorToolkit> InAssetEditor)
	{
		return MakeShareable(new FAnimationModifiersTabSummoner(InAssetEditor)); 
	}
};

#undef LOCTEXT_NAMESPACE // AnimationModifiersModule