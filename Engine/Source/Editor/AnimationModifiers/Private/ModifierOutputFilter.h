// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/StringOutputDevice.h"
#include "UObject/NameTypes.h"

class FCategoryLogOutputFilter : public FStringOutputDevice
{
	typedef FStringOutputDevice Super;

	TArray<FName> CategoryNames;
	bool bError;
	bool bWarning;
public:
	FCategoryLogOutputFilter(const TCHAR* OutputDeviceName = TEXT(""))
		: Super(OutputDeviceName), bError(false), bWarning(false)
	{}

	void AddCategoryName(const FName& CategoryName)
	{
		CategoryNames.AddUnique(CategoryName);
	}

	void RemoveCategoryName(const FName& CategoryName)
	{
		CategoryNames.Remove(CategoryName);
	}

	bool ContainsErrors()
	{
		return bError;
	}

	bool ContainsWarnings()
	{
		return bWarning;
	}

	virtual void Serialize(const TCHAR* InData, ELogVerbosity::Type Verbosity, const class FName& Category) override
	{
		if (CategoryNames.Contains(Category))
		{
			FString::operator+=(ToString(Verbosity));
			FString::operator+=((TCHAR*)TEXT(": "));
			Super::Serialize(InData, Verbosity, Category);

			bError |= (Verbosity == ELogVerbosity::Error);
			bWarning |= (Verbosity == ELogVerbosity::Warning);
		}
	}
};