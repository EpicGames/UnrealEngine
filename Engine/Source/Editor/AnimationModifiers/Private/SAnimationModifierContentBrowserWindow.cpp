// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimationModifierContentBrowserWindow.h"

#include "Animation/AnimSequence.h"
#include "AnimationModifier.h"
#include "AnimationModifierHelpers.h"
#include "AnimationModifiersAssetUserData.h"
#include "ClassViewerModule.h"
#include "Delegates/Delegate.h"
#include "DetailsViewArgs.h"
#include "Framework/SlateDelegates.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "IDetailsView.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SModifierListview.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"
#include "Templates/SubclassOf.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

struct FGeometry;

#define LOCTEXT_NAMESPACE "AnimationModifierContentBrowserWindow"

////////////////////////////////////////////////////////////////////////////////////
// Add Animation Modifier(s) widget
////////////////////////////////////////////////////////////////////////////////////

void SAnimationModifierContentBrowserWindow::Construct(const FArguments& InArgs)
{
	CreateInstanceDetailsView();

	WidgetWindow = InArgs._WidgetWindow;
	AnimSequences = InArgs._AnimSequences;

	FOnGetContent GetContent = FOnGetContent::CreateLambda(
		[this]()
	{
		return FAnimationModifierHelpers::GetModifierPicker(FOnClassPicked::CreateRaw(this, &SAnimationModifierContentBrowserWindow::OnModifierPicked));
	});

	this->ChildSlot
	[
		SNew(SOverlay)		
		+SOverlay::Slot()
		[				
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.Padding(2.0f)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(3.0f, 3.0f)
					.AutoWidth()
					[
						SAssignNew(AddModifierCombobox, SComboButton)
						.OnGetMenuContent(GetContent)
						.ButtonContent()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("AnimationModifierWindow_AddModifier", "Add Modifier"))
						]
					]
				]
			]

			+ SVerticalBox::Slot()
			[
				SNew(SBorder)
				.Padding(2.0f)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[				
					SNew(SSplitter)
					.Orientation(EOrientation::Orient_Vertical)

					+ SSplitter::Slot()
					.Value(.5f)
					[
						SNew(SBox)
						.Padding(2.0f)
						[
							SAssignNew(ModifierListView, SModifierListView)
							.Items(&ModifierItems)
							.InstanceDetailsView(ModifierInstanceDetailsView)
							.OnRemoveModifier(FOnModifierArray::CreateSP(this, &SAnimationModifierContentBrowserWindow::RemoveModifiersCallback))
						]
					]

					+ SSplitter::Slot()
					.Value(.5f)
					[	
						SNew(SBox)
						.Padding(2.0f)
						[
							ModifierInstanceDetailsView->AsShared()
						]
					]
				]
			]	

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(2)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(2)
				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("AnimationModifierWindow_Import", "Apply"))
					.ToolTipText(LOCTEXT("AnimationModifierWindow_Import_ToolTip", "Apply adding modifiers(s)."))
					.IsEnabled(this, &SAnimationModifierContentBrowserWindow::CanApply)
					.OnClicked(this, &SAnimationModifierContentBrowserWindow::OnApply)
				]
				+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("AnimationModifierWindow_Cancel", "Cancel"))
					.ToolTipText(LOCTEXT("AnimationModifierWindow_Cancel_ToolTip", "Cancels adding modifiers(s)."))
					.OnClicked(this, &SAnimationModifierContentBrowserWindow::OnCancel)
				]
			]	
		]
	];
}

FReply SAnimationModifierContentBrowserWindow::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		return OnCancel();
	}

	return FReply::Unhandled();
}

void SAnimationModifierContentBrowserWindow::RemoveModifiersCallback(const TArray<TWeakObjectPtr<UAnimationModifier>>& ModifiersToRemove)
{
	Modifiers.RemoveAll([&ModifiersToRemove](UAnimationModifier* Modifier) { return ModifiersToRemove.Contains(Modifier); });
	ModifierItems.RemoveAll([&ModifiersToRemove](TSharedPtr<FModifierListviewItem> ModifierItem) { return ModifiersToRemove.Contains(ModifierItem->Instance); });
	ModifierListView->Refresh();
}

void SAnimationModifierContentBrowserWindow::OnModifierPicked(UClass* PickedClass)
{	
	UAnimationModifier* Processor = FAnimationModifierHelpers::CreateModifierInstance(GetTransientPackage(), PickedClass);

	Modifiers.Add(Processor);

	FModifierListviewItem* Item = new FModifierListviewItem();
	Item->Instance = Processor;
	Item->Class = Processor->GetClass();
	Item->Index = Modifiers.Num() - 1;
	Item->OuterClass = nullptr;
	ModifierItems.Add(ModifierListviewItem(Item));

	// Close the combo box
	AddModifierCombobox->SetIsOpen(false);

	ModifierListView->Refresh();
}

void SAnimationModifierContentBrowserWindow::CreateInstanceDetailsView()
{
	// Create a property view
	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
	DetailsViewArgs.bShowOptions = false;

	ModifierInstanceDetailsView = EditModule.CreateDetailView(DetailsViewArgs);
	ModifierInstanceDetailsView->SetDisableCustomDetailLayouts(true);
}

FReply SAnimationModifierContentBrowserWindow::OnApply()
{
	const FScopedTransaction Transaction(LOCTEXT("UndoAction_ApplyModifiers", "Applying Animation Modifier(s) to Animation Sequence(s)"));

	TArray<UAnimationModifiersAssetUserData*> AssetUserData;

	bool bCloseWindow = true;

	// Retrieve or create asset user data 
	for (UAnimSequence* AnimationSequence : AnimSequences)
	{
		UAnimationModifiersAssetUserData* UserData = AnimationSequence->GetAssetUserData<UAnimationModifiersAssetUserData>();
		if (!UserData)
		{
			UserData = NewObject<UAnimationModifiersAssetUserData>(AnimationSequence, UAnimationModifiersAssetUserData::StaticClass());
			checkf(UserData, TEXT("Unable to instantiate AssetUserData class"));
			UserData->SetFlags(RF_Transactional);
			AnimationSequence->AddAssetUserData(UserData);
		}

		AssetUserData.Add(UserData);
	}

	UE::Anim::FApplyModifiersScope Scope;

	// For each added modifier create add a new instance to each of the user data entries, using the one(s) set up in the window as template(s)
	for (UAnimationModifier* Modifier : Modifiers)
	{
		for (UAnimationModifiersAssetUserData* UserData : AssetUserData)
		{
			const bool bAlreadyContainsModifier = UserData->GetAnimationModifierInstances().ContainsByPredicate([Modifier](UAnimationModifier* TestModifier) { return Modifier->GetClass() == TestModifier->GetClass(); });

			bool bUserInputResult = true;
			if (bAlreadyContainsModifier)
			{
				FText MessageFormat = LOCTEXT("AnimationModifierWindow_AlreadyContainsModifierDialogText", "{0} already contains Animation Modifier {1}, are you sure you want to add another instance?");
				FText MessageTitle = LOCTEXT("AnimationModifierWindow_AlreadyContainsModifierTitle", "Already contains Animation Modifier!");
				bUserInputResult = FMessageDialog::Open(EAppMsgType::YesNo, FText::FormatOrdered(MessageFormat, FText::FromString(UserData->GetOuter()->GetName()), FText::FromString(Modifier->GetClass()->GetName())), MessageTitle) == EAppReturnType::Yes;
			}
			
			bCloseWindow = bUserInputResult;
			if (!bAlreadyContainsModifier || bUserInputResult)
			{
				UObject* Outer = UserData;
				UAnimationModifier* Processor = FAnimationModifierHelpers::CreateModifierInstance(Outer, Modifier->GetClass(), Modifier);
				UserData->Modify();
				UserData->AddAnimationModifier(Processor);
			}
		}
	}

	/** For each user data retrieve all modifiers and apply them */
	for (int32 Index = 0; Index < AssetUserData.Num(); ++Index)
	{
		UAnimationModifiersAssetUserData* UserData = AssetUserData[Index];		
		if (UserData)
		{
			UAnimSequence* AnimSequence = AnimSequences[Index];
			AnimSequence->Modify();

			const TArray<UAnimationModifier*>& ModifierInstances = UserData->GetAnimationModifierInstances();
			for (UAnimationModifier* Modifier : ModifierInstances)
			{
				Modifier->ApplyToAnimationSequence(AnimSequence);
			}
		}
	}

	if (bCloseWindow && WidgetWindow.IsValid())
	{
		WidgetWindow.Pin()->RequestDestroyWindow();
	}

	return FReply::Handled();
}

FReply SAnimationModifierContentBrowserWindow::OnCancel()
{
	if (WidgetWindow.IsValid())
	{
		WidgetWindow.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

bool SAnimationModifierContentBrowserWindow::CanApply() const
{
	return Modifiers.Num() > 0;
}

////////////////////////////////////////////////////////////////////////////////////
// Remove Animation Modifier(s) widget
////////////////////////////////////////////////////////////////////////////////////

/** Widget used to display a list of sequences that will be modified after an animation modifier application */
class SAnimSequencesToBeModifiedViewer : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SAnimSequencesToBeModifiedViewer)
	{
	}

	SLATE_END_ARGS()

	void Construct(const FArguments& Args)
	{
		AnimSequencesToBeModifiedListView = SNew(SListView<UAnimSequence*>)
				.ListItemsSource(&AnimSequencesToBeModifiedList)
				.SelectionMode(ESelectionMode::None)
				.OnGenerateRow(this, &SAnimSequencesToBeModifiedViewer::OnGenerateWidgetForRow);

		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				AnimSequencesToBeModifiedListView.ToSharedRef()
			]
		];
	};

	void UpdateToBeModifiedAnimSequencesList(const TArray<UAnimSequence*>& InAnimSequences, const UAnimationModifier * InModifier)
	{
		AnimSequencesToBeModifiedList.Empty();

		if (InModifier != nullptr)
		{
			for (UAnimSequence* AnimationSequence : InAnimSequences)
			{
				if (const UAnimationModifiersAssetUserData* UserData = AnimationSequence->GetAssetUserData<UAnimationModifiersAssetUserData>())
				{
					const bool bIsModifierFoundInAnimSequence = UserData->GetAnimationModifierInstances().ContainsByPredicate(([InModifier](const UAnimationModifier* TestModifier)
					{
						return InModifier->GetClass() == TestModifier->GetClass();
					}));
				
					if (bIsModifierFoundInAnimSequence)
					{
						AnimSequencesToBeModifiedList.Push(AnimationSequence);
					}
				}
			}
		}
		
		AnimSequencesToBeModifiedListView->RequestListRefresh();
	}
	
private:

	TSharedRef<ITableRow> OnGenerateWidgetForRow(UAnimSequence* InItem, const TSharedRef<STableViewBase>& OwnerTable) const
	{
		const FSlateBrush* ClassIcon = FSlateIconFinder::FindIconBrushForClass(UAnimSequence::StaticClass());
		
		return SNew(STableRow<UAnimSequence*>, OwnerTable)
		[
			SNew(SHorizontalBox)

			// Logo
			+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding( 6.0f, 2.0f, 6.0f, 2.0f )
				[
					SNew( SImage )
					.Image(	ClassIcon )
					.Visibility( ClassIcon != FAppStyle::GetDefaultBrush()? EVisibility::Visible : EVisibility::Collapsed )
				]

			// Display name
			+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding( 0.0f, 3.0f, 6.0f, 3.0f )
				.VAlign(VAlign_Center)
				[
					SNew( STextBlock )
						.Text( FText::FromString(InItem->GetName()) )
						.ColorAndOpacity(FSlateColor::UseForeground())
				]
		];
	}

	TSharedPtr<SListView<UAnimSequence*>> AnimSequencesToBeModifiedListView;
	TArray<UAnimSequence*> AnimSequencesToBeModifiedList;
};

void SRemoveAnimationModifierContentBrowserWindow::Construct(const FArguments& InArgs)
{
	WidgetWindow = InArgs._WidgetWindow;
	AnimSequences = InArgs._AnimSequences;
	
	this->ChildSlot
	[
		SNew(SOverlay)		
		+SOverlay::Slot()
		[				
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.Padding(2.0f)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(3.0f, 3.0f)
					.AutoWidth()
					[
						SAssignNew(AddModifierCombobox, SComboButton)
						.MenuContent()
						[
							SNew(SBox)
							.WidthOverride(280)
							[
								SNew(SVerticalBox)
								+ SVerticalBox::Slot()
								.AutoHeight()
								.MaxHeight(500)
								[
									GenerateAnimationModifierPicker()
								]
							]
						]
						.ButtonContent()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("AnimationModifierWindow_RemoveModifier", "Remove Modifier"))
						]
					]
				]
			]
			
			+ SVerticalBox::Slot()
			[
				SNew(SBorder)
				.Padding(2.0f)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[				
					SNew(SSplitter)
					.Orientation(EOrientation::Orient_Vertical)

					+ SSplitter::Slot()
					.Value(.5f)
					[
						SNew(SBox)
						.Padding(2.0f)
						[
							SAssignNew(ModifierListView, SModifierListView)
							.Items(&ModifierItems)
							.InstanceDetailsView(ModifierInstanceDetailsView)
							.OnRemoveModifier(FOnModifierArray::CreateSP(this, &SRemoveAnimationModifierContentBrowserWindow::RemoveModifiersCallback))
							.OnSelectedModifierChanged_Lambda([this](const TWeakObjectPtr<UAnimationModifier>& InSelectedModifier)
							{
								AnimSequencesToBeModifiedListViewer->UpdateToBeModifiedAnimSequencesList(AnimSequences, InSelectedModifier.Get());
							})
						]
					]

					// Show sequences that this modifier is affecting.
					+ SSplitter::Slot()
					.Value(.5f)
					[
						SNew(SVerticalBox)

						+SVerticalBox::Slot()
						.AutoHeight()
						.Padding(6.f, 4.f)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("AnimationModifierWindow_AffectedTitle", "Sequence(s) affected by the selected Animation Modifier"))
						]
						
						+SVerticalBox::Slot()
						.Padding(2.0f)
						[
							SAssignNew(AnimSequencesToBeModifiedListViewer, SAnimSequencesToBeModifiedViewer)
						]
					]
				]
			]	

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(2)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(2)
				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("AnimationModifierWindow_ApplyRemove", "Apply"))
					.ToolTipText(LOCTEXT("AnimationModifierWindow_ApplyRemove_ToolTip", "Apply remove modifiers(s)."))
					.IsEnabled(this, &SRemoveAnimationModifierContentBrowserWindow::CanApply)
					.OnClicked(this, &SRemoveAnimationModifierContentBrowserWindow::OnApply)
				]
				+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("AnimationModifierWindow_CancelRemove", "Cancel"))
					.ToolTipText(LOCTEXT("AnimationModifierWindow_CancelRemove_ToolTip", "Cancels removing modifiers(s)."))
					.OnClicked(this, &SRemoveAnimationModifierContentBrowserWindow::OnCancel)
				]
			]	
		]
	];
}

FReply SRemoveAnimationModifierContentBrowserWindow::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		return OnCancel();
	}

	return FReply::Unhandled();
}

void SRemoveAnimationModifierContentBrowserWindow::OnModifierPicked(UClass* PickedClass)
{
	UAnimationModifier* Processor = FAnimationModifierHelpers::CreateModifierInstance(GetTransientPackage(), PickedClass);

	Modifiers.Add(Processor);

	FModifierListviewItem* Item = new FModifierListviewItem();
	Item->Instance = Processor;
	Item->Class = Processor->GetClass();
	Item->Index = Modifiers.Num() - 1;
	Item->OuterClass = nullptr;
	ModifierItems.Add(ModifierListviewItem(Item));

	// Close the combo box
	AddModifierCombobox->SetIsOpen(false);

	ModifierListView->Refresh();
}

void SRemoveAnimationModifierContentBrowserWindow::RemoveModifiersCallback(const TArray<TWeakObjectPtr<UAnimationModifier>>& ModifiersToRemove)
{
	Modifiers.RemoveAll([&ModifiersToRemove](UAnimationModifier* Modifier) { return ModifiersToRemove.Contains(Modifier); });
	ModifierItems.RemoveAll([&ModifiersToRemove](TSharedPtr<FModifierListviewItem> ModifierItem) { return ModifiersToRemove.Contains(ModifierItem->Instance); });
	ModifierListView->Refresh();
}

TSharedRef<SWidget> SRemoveAnimationModifierContentBrowserWindow::GenerateAnimationModifierPicker()
{
	/** Constrain(s) a picker's classes to those of currently applied pickers */
	class FRemoveModifierClassFilter : public IClassViewerFilter
	{
	public:
		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			return InClass->IsChildOf(UAnimationModifier::StaticClass()) && AllowList.Contains(InClass);
		}

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			return false;
		}
		
		TSet<UClass*> AllowList;
	};

	// Customize class picker
	FClassViewerInitializationOptions Options;
	Options.bShowUnloadedBlueprints = true;
	Options.bShowNoneOption = false;
	const TSharedRef<FRemoveModifierClassFilter> ClassFilter = MakeShared<FRemoveModifierClassFilter>();

	// Query every applied Animation Modifier(s)
	for (UAnimSequence* AnimationSequence : AnimSequences)
	{
		if (UAnimationModifiersAssetUserData* UserData = AnimationSequence->GetAssetUserData<UAnimationModifiersAssetUserData>())
		{
			for (UAnimationModifier* const& Modifier : UserData->GetAnimationModifierInstances())
			{
				ClassFilter->AllowList.Add(Modifier->GetClass());
			}
		}
	}
	Options.ClassFilters.Add(ClassFilter);

	return FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(Options, FOnClassPicked::CreateRaw(this, &SRemoveAnimationModifierContentBrowserWindow::OnModifierPicked));
}

FReply SRemoveAnimationModifierContentBrowserWindow::OnApply()
{
	// Get user confirmation for action(s)
	const bool bShouldRevert = FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("RemoveAndRevertPopupText", "Should the Animation Modifiers be reverted before removing them?"), LOCTEXT("RemoveAndRevertPopupTitle", "Revert before Removing?")) == EAppReturnType::Yes;
	const bool bShouldRemove = FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("RemoveAnimModifiers", "Are you sure you want to remove the anim modifiers from all the selected animation sequences?"), LOCTEXT("RemoveAnimModifiersPopupTitle", "Are you sure?")) == EAppReturnType::Yes;
	
	if (!bShouldRemove)
	{
		return FReply::Handled();
	}
	
	FScopedTransaction Transaction(LOCTEXT("RemoveModifiersTransaction", "Removing Animation Modifier(s)"));
	
	for (UAnimSequence* AnimationSequence : AnimSequences)
	{
		if (UAnimationModifiersAssetUserData* UserData = AnimationSequence->GetAssetUserData<UAnimationModifiersAssetUserData>())
		{
			const TArray<UAnimationModifier*>& AnimModifierInstancesInUserData = UserData->GetAnimationModifierInstances();
			for (const UAnimationModifier* Modifier : Modifiers)
			{
				checkf(Modifier, TEXT("Invalid selected modifier"));

				UAnimationModifier* const* ModifierInstanceInUserData = AnimModifierInstancesInUserData.FindByPredicate([Modifier](const UAnimationModifier* TestModifier)
				{
					return Modifier->GetClass() == TestModifier->GetClass();
				});

				if (ModifierInstanceInUserData == nullptr)
				{
					continue;
				}

				if (bShouldRevert)
				{
					(*ModifierInstanceInUserData)->RevertFromAnimationSequence(AnimationSequence);

					// Revert can not fail, thus we can always mark reverted
					if ((*ModifierInstanceInUserData)->HasLegacyPreviousAppliedModifierOnSkeleton())
					{
						checkf(AnimationSequence->GetSkeleton() != nullptr, TEXT("Invalid skeleton for anim sequence"));
						(*ModifierInstanceInUserData)->RemoveLegacyPreviousAppliedModifierOnSkeleton(AnimationSequence->GetSkeleton());
					}
				}

				UserData->Modify();
				UserData->RemoveAnimationModifierInstance(*ModifierInstanceInUserData);
			}
		}
	}
	
	if (WidgetWindow.IsValid())
	{
		WidgetWindow.Pin()->RequestDestroyWindow();
	}

	return FReply::Handled();
}

FReply SRemoveAnimationModifierContentBrowserWindow::OnCancel() const
{
	if (WidgetWindow.IsValid())
	{
		WidgetWindow.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

bool SRemoveAnimationModifierContentBrowserWindow::CanApply() const
{
	return Modifiers.Num() > 0;
}

#undef LOCTEXT_NAMESPACE // "AnimationModifierContentBrowserWindow"
