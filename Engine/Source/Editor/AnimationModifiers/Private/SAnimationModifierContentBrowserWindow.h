// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Input/Reply.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class IDetailsView;
class SMenuAnchor;
class SModifierListView;
class SWindow;
class UAnimSequence;
class UAnimationModifier;
class UClass;
struct FGeometry;
struct FKeyEvent;
struct FModifierListviewItem;

/** UI slate widget allowing the user to add Animation Modifier(s) to a selection of Animation Sequences */
class SAnimationModifierContentBrowserWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAnimationModifierContentBrowserWindow)
		: _WidgetWindow()		
	{}			
		SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
		SLATE_ARGUMENT(TArray<UAnimSequence*>, AnimSequences)
	SLATE_END_ARGS()

public:
	SAnimationModifierContentBrowserWindow() {}
	void Construct(const FArguments& InArgs);
	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
protected:
	/** Callback for when the user picks a specific animation modifier class */
	void OnModifierPicked(UClass* PickedClass);
	/** Callback for when the user wants to remove modifier(s) from the listview */
	void RemoveModifiersCallback(const TArray<TWeakObjectPtr<UAnimationModifier>>& ModifiersToRemove);

	/** Creates the details view widget used to show Animation Modifier object details */
	void CreateInstanceDetailsView();

	/** Button callback, this applies all currently set up Animation Modifiers to the previously selected Animation Sequences */
	FReply OnApply();
	/** Button callback, closes the dialog/window */
	FReply OnCancel();
	/** Check to see whether or not the user can apply the modifiers in the current state */
	bool CanApply() const;
private:
	/** Window owning this window */
	TWeakPtr<SWindow> WidgetWindow;
	
	TSharedPtr<IDetailsView> ModifierInstanceDetailsView;
	TSharedPtr<SMenuAnchor> AddModifierCombobox;
	TSharedPtr<SModifierListView> ModifierListView;

	/** Data structures used by the Modifier List View widget */
	TArray<TSharedPtr<FModifierListviewItem>> ModifierItems;	
	/** Current set of Animation Modifiers that would be added during Apply */
	TArray<UAnimationModifier*> Modifiers;
	/** Previously user-selected Animation Sequences */
	TArray<UAnimSequence*> AnimSequences;
};

class SAnimSequencesToBeModifiedViewer;

/** UI slate widget allowing the user to remove Animation Modifier(s) to a selection of Animation Sequences */
class SRemoveAnimationModifierContentBrowserWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRemoveAnimationModifierContentBrowserWindow)
		: _WidgetWindow()		
	{}			
	SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
	SLATE_ARGUMENT(TArray<UAnimSequence*>, AnimSequences)
SLATE_END_ARGS()

public:
	SRemoveAnimationModifierContentBrowserWindow() {}
	void Construct(const FArguments& InArgs);
	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
protected:

	/** Callback for when the user picks a specific animation modifier class */
	void OnModifierPicked(UClass* PickedClass);
	
	/** Callback for when the user wants to remove modifier(s) from the listview */
	void RemoveModifiersCallback(const TArray<TWeakObjectPtr<UAnimationModifier>>& ModifiersToRemove);

	/** Creates a picker that only displays Animation Modifier(s) currently applied to the selected Animation Sequences  */
	TSharedRef<SWidget> GenerateAnimationModifierPicker();
	
	/** Button callback, this applies all currently set up Animation Modifiers to the previously selected Animation Sequences */
	FReply OnApply();
	/** Button callback, closes the dialog/window */
	FReply OnCancel() const;
	/** Check to see whether or not the user can apply the modifiers in the current state */
	bool CanApply() const;
	
private:
	/** Window owning this window */
	TWeakPtr<SWindow> WidgetWindow;
	
	TSharedPtr<IDetailsView> ModifierInstanceDetailsView;
	TSharedPtr<SMenuAnchor> AddModifierCombobox;
	TSharedPtr<SModifierListView> ModifierListView;
	TSharedPtr<SAnimSequencesToBeModifiedViewer> AnimSequencesToBeModifiedListViewer;
	
	/** Data structures used by the Modifier List View widget */
	TArray<TSharedPtr<FModifierListviewItem>> ModifierItems;	
	/** Current set of Animation Modifiers that would be added during Apply */
	TArray<UAnimationModifier*> Modifiers;
	/** Previously user-selected Animation Sequences */
	TArray<UAnimSequence*> AnimSequences;
};