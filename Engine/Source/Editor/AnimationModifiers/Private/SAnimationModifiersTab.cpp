// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimationModifiersTab.h"

#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "AnimationModifier.h"
#include "AnimationModifierHelpers.h"
#include "AnimationModifiersAssetUserData.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "ClassViewerModule.h"
#include "Delegates/Delegate.h"
#include "DetailsViewArgs.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Framework/SlateDelegates.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "IDetailsView.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SModifierListview.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"

class UClass;
struct FGeometry;

#define LOCTEXT_NAMESPACE "SAnimationModifiersTab"

SAnimationModifiersTab::SAnimationModifiersTab()
	: Skeleton(nullptr), AnimationSequence(nullptr), AssetUserData(nullptr), bDirty(false)
{	
}

SAnimationModifiersTab::~SAnimationModifiersTab()
{
	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetOpenedInEditor().RemoveAll(this);	
	}
}

void SAnimationModifiersTab::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bDirty)
	{
		RetrieveModifierData();
		ModifierListView->Refresh();
		bDirty = false;
	}
}

void SAnimationModifiersTab::PostUndo(bool bSuccess)
{
	Refresh();
}

void SAnimationModifiersTab::PostRedo(bool bSuccess)
{
	Refresh();
}

void SAnimationModifiersTab::Construct(const FArguments& InArgs)
{
	HostingApp = InArgs._InHostingApp;
	
	// Retrieve asset and modifier data
	RetrieveAnimationAsset();
	RetrieveModifierData();
		
	CreateInstanceDetailsView();

	FOnGetContent GetContent = FOnGetContent::CreateLambda(
		[this]()
	{
		return FAnimationModifierHelpers::GetModifierPicker(FOnClassPicked::CreateRaw(this, &SAnimationModifiersTab::OnModifierPicked));
	});
	
	this->ChildSlot
	[
		SNew(SOverlay)		
		+SOverlay::Slot()
		[				
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.Padding(2.0f)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(3.0f, 3.0f)
					.AutoWidth()
					[
						SAssignNew(AddModifierCombobox, SComboButton)
						.OnGetMenuContent(GetContent)
						.ButtonContent()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("AddModifier", "Add Modifier"))
						]
					]

					+ SHorizontalBox::Slot()
					.Padding(3.0f, 3.0f)
					.AutoWidth()
					[
						SNew(SButton)
						.OnClicked(this, &SAnimationModifiersTab::OnApplyAllModifiersClicked)
						.ContentPadding(FMargin(5))
						.Content()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ApplyAllModifiers", "Apply All Modifiers"))
						]
					]
				]
			]

			+ SVerticalBox::Slot()
			[
				SNew(SBorder)
				.Padding(2.0f)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[				
					SNew(SSplitter)
					.Orientation(EOrientation::Orient_Vertical)

					+ SSplitter::Slot()
					.Value(.5f)
					[
						SNew(SBox)
						.Padding(2.0f)
						[
							SAssignNew(ModifierListView, SModifierListView)
							.Items(&ModifierItems)
							.InstanceDetailsView(ModifierInstanceDetailsView)
							.OnApplyModifier(FOnModifierArray::CreateSP(this, &SAnimationModifiersTab::OnApplyModifier))
							.OnCanRevertModifier(FOnCanModifierArray::CreateSP(this, &SAnimationModifiersTab::OnCanRevertModifier))
							.OnRevertModifier(FOnModifierArray::CreateSP(this, &SAnimationModifiersTab::OnRevertModifier))
							.OnRemoveModifier(FOnModifierArray::CreateSP(this, &SAnimationModifiersTab::OnRemoveModifier))
							.OnOpenModifier(FOnSingleModifier::CreateSP(this, &SAnimationModifiersTab::OnOpenModifier))
							.OnMoveUpModifier(FOnSingleModifier::CreateSP(this, &SAnimationModifiersTab::OnMoveModifierUp))
							.OnMoveDownModifier(FOnSingleModifier::CreateSP(this, &SAnimationModifiersTab::OnMoveModifierDown))
						]
					]

					+ SSplitter::Slot()
					.Value(.5f)
					[	
						SNew(SBox)
						.Padding(2.0f)
						[
							ModifierInstanceDetailsView->AsShared()
						]
					]
				]
			]	
		]
	];

	// Ensure that this tab is only enabled if we have a valid AssetUserData instance
	this->ChildSlot.GetWidget()->SetEnabled(TAttribute<bool>::Create([this]() { return AssetUserData != nullptr; }));

	// Register delegates
	if (GEditor)
	{
		GEditor->RegisterForUndo(this);
	}
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetOpenedInEditor().AddSP(this, &SAnimationModifiersTab::OnAssetOpened);
}

void SAnimationModifiersTab::OnModifierPicked(UClass* PickedClass)
{
	FScopedTransaction Transaction(LOCTEXT("AddModifierTransaction", "Adding Animation Modifier"));
	
	UObject* Outer = AssetUserData;
	UAnimationModifier* Processor = FAnimationModifierHelpers::CreateModifierInstance(Outer, PickedClass);
	AssetUserData->Modify();
	AssetUserData->AddAnimationModifier(Processor);
	
	// Close the combo box
	AddModifierCombobox->SetIsOpen(false);

	// Refresh the UI
	Refresh();
}

void SAnimationModifiersTab::RetrieveAnimationAsset()
{
	TSharedPtr<FAssetEditorToolkit> AssetEditor = HostingApp.Pin();
	const TArray<UObject*>* EditedObjects = AssetEditor->GetObjectsCurrentlyBeingEdited();
	AssetUserData = nullptr;

	if (EditedObjects)
	{
		// Try and find an AnimSequence or Skeleton asset in the currently being edited objects, and retrieve or add ModfiersAssetUserDat
		for (UObject* Object : (*EditedObjects))
		{
			if (Object->IsA<UAnimSequence>())
			{
				AnimationSequence = Cast<UAnimSequence>(Object);
				AssetUserData = FAnimationModifierHelpers::RetrieveOrCreateModifierUserData(AnimationSequence);
				
				break;
			}
			else if (Object->IsA<USkeleton>())
			{
				Skeleton = Cast<USkeleton>(Object);
				AssetUserData = FAnimationModifierHelpers::RetrieveOrCreateModifierUserData(Skeleton);
				break;
			}
		}
	}
}

void SAnimationModifiersTab::CreateInstanceDetailsView()
{
	// Create a property view
	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
	DetailsViewArgs.bShowOptions = false;	

	ModifierInstanceDetailsView = EditModule.CreateDetailView(DetailsViewArgs);	
	ModifierInstanceDetailsView->SetDisableCustomDetailLayouts(true);
}

FReply SAnimationModifiersTab::OnApplyAllModifiersClicked()
{
	FScopedTransaction Transaction(LOCTEXT("ApplyAllModifiersTransaction", "Applying All Animation Modifier(s)"));
	const TArray<UAnimationModifier*>& ModifierInstances = AssetUserData->GetAnimationModifierInstances();	
	ApplyModifiers(ModifierInstances);
	return FReply::Handled();
}

void SAnimationModifiersTab::OnApplyModifier(const TArray<TWeakObjectPtr<UAnimationModifier>>& Instances)
{
	TArray<UAnimationModifier*> ModifierInstances;
	for (TWeakObjectPtr<UAnimationModifier> InstancePtr : Instances)
	{
		checkf(InstancePtr.IsValid(), TEXT("Invalid weak object ptr to modifier instance"));
		UAnimationModifier* Instance = InstancePtr.Get();
		ModifierInstances.Add(Instance);	
	}

	FScopedTransaction Transaction(LOCTEXT("ApplyModifiersTransaction", "Applying Animation Modifier(s)"));
	ApplyModifiers(ModifierInstances);
	Refresh();
}

void SAnimationModifiersTab::FindAnimSequencesForSkeleton(TArray<UAnimSequence *>& ReferencedAnimSequences)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(FName("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	// Search for referencing packages to the currently open skeleton
	TArray<FAssetIdentifier> Referencers;
	AssetRegistry.GetReferencers(Skeleton->GetOuter()->GetFName(), Referencers);
	for (const FAssetIdentifier& Identifier : Referencers)
	{
		TArray<FAssetData> Assets;
		AssetRegistry.GetAssetsByPackageName(Identifier.PackageName, Assets);

		for (const FAssetData& Asset : Assets)
		{
			// Only add assets whos class is of UAnimSequence
			if (Asset.IsInstanceOf(UAnimSequence::StaticClass()))
			{
				ReferencedAnimSequences.Add(CastChecked<UAnimSequence>(Asset.GetAsset()));
			}
		}
	}
}

void SAnimationModifiersTab::OnRevertModifier(const TArray<TWeakObjectPtr<UAnimationModifier>>& Instances)
{
	TArray<UAnimationModifier*> ModifierInstances;
	for (TWeakObjectPtr<UAnimationModifier> InstancePtr : Instances)
	{
		checkf(InstancePtr.IsValid(), TEXT("Invalid weak object ptr to modifier instance"));
		UAnimationModifier* Instance = InstancePtr.Get();
		ModifierInstances.Add(Instance);
	}
	
	// if modifier has bReapplyPostOwnerChange=true, we need to stop that from re-applying
	UE::Anim::FApplyModifiersScope Scope;
	FScopedTransaction Transaction(LOCTEXT("RevertModifiersTransaction", "Reverting Animation Modifier(s)"));
	RevertModifiers(ModifierInstances);
	Refresh();
}

bool SAnimationModifiersTab::OnCanRevertModifier(const TArray<TWeakObjectPtr<UAnimationModifier>>& Instances)
{
	bool bCanRevert = false;

	if (AnimationSequence)
	{
		for (const TWeakObjectPtr<UAnimationModifier>& InstancePtr : Instances)
		{
			checkf(InstancePtr.IsValid(), TEXT("Invalid weak object ptr to modifier instance"));
			const UAnimationModifier* Modifier = InstancePtr.Get();

			// At least one instance has to be revert-able
			if (Modifier->CanRevert(AnimationSequence))
			{
				bCanRevert = true;
				break;
			}
		}
	}
	else if (Skeleton) 
	{
		// Check for modifiers applied on previous version
		// Where asset registry tags were not written
		for (const TWeakObjectPtr<UAnimationModifier>& Modifier : Instances)
		{
			if (Modifier->HasLegacyPreviousAppliedModifierOnSkeleton())
			{
				bCanRevert = true;
				return true;
			}
		}

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(FName("AssetRegistry"));
		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

		TSet<FString> Modifiers;
		Algo::Transform(Instances, Modifiers, [](const TWeakObjectPtr<UAnimationModifier>& Ptr) 
		{
			return Ptr.Get()->GetName();
		});

		FARFilter Filter;
		Filter.ClassPaths.Add(UAnimSequence::StaticClass()->GetClassPathName());
		Filter.TagsAndValues.Add(FName{"Skeleton"}) = FAssetData(Skeleton).GetExportTextName();

		AssetRegistry.EnumerateAssets(Filter, [&](const FAssetData& Asset) 
		{
			FString Tag = Asset.GetTagValueRef<FString>(UAnimationModifier::AnimationModifiersTag);

			FAnimationModifierHelpers::EnumerateAnimationModifierTags(Tag, [&](FStringView Name, FGuid Revision) 
			{
				bCanRevert = Modifiers.ContainsByHash(GetTypeHash(Name), Name);
				return !bCanRevert; // Break if bCanRevert is already true
			});

			return !bCanRevert; // Break if bCanRevert is already true
		});

	}
	return bCanRevert;
}

void SAnimationModifiersTab::OnRemoveModifier(const TArray<TWeakObjectPtr<UAnimationModifier>>& Instances)
{
	const bool bShouldRevert = FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("RemoveAndRevertPopupText", "Should the Modifiers be reverted before removing them?"), LOCTEXT("RemoveAndRevertPopupTitle", "Revert before Removing")) == EAppReturnType::Yes;

	FScopedTransaction Transaction(LOCTEXT("RemoveModifiersTransaction", "Removing Animation Modifier(s)"));	
	AssetUserData->Modify();

	if (bShouldRevert)
	{
		// if modifier has bReapplyPostOwnerChange=true, we need to stop that from re-applying
		UE::Anim::FApplyModifiersScope Scope;
		TArray<UAnimationModifier*> ModifierInstances;
		for (TWeakObjectPtr<UAnimationModifier> InstancePtr : Instances)
		{
			checkf(InstancePtr.IsValid(), TEXT("Invalid weak object ptr to modifier instance"));
			UAnimationModifier* Instance = InstancePtr.Get();
			ModifierInstances.Add(Instance);
		}

		RevertModifiers(ModifierInstances);
	}

	for (TWeakObjectPtr<UAnimationModifier> InstancePtr : Instances)
	{
		checkf(InstancePtr.IsValid(), TEXT("Invalid weak object ptr to modifier instance"));

		UAnimationModifier* Instance = InstancePtr.Get();
		AssetUserData->Modify();
		AssetUserData->RemoveAnimationModifierInstance(Instance);
	}

	Refresh();
}

void SAnimationModifiersTab::OnOpenModifier(const TWeakObjectPtr<UAnimationModifier>& Instance)
{
	checkf(Instance.IsValid(), TEXT("Invalid weak object ptr to modifier instance"));
	const UAnimationModifier* ModifierInstance = Instance.Get();
	const UBlueprintGeneratedClass* BPGeneratedClass = Cast<UBlueprintGeneratedClass>(ModifierInstance->GetClass());

	if (BPGeneratedClass && BPGeneratedClass->ClassGeneratedBy)
	{
		UBlueprint* Blueprint = Cast<UBlueprint>(BPGeneratedClass->ClassGeneratedBy);
		if (Blueprint)
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Blueprint);
		}
	}
}

void SAnimationModifiersTab::OnMoveModifierUp(const TWeakObjectPtr<UAnimationModifier>& Instance)
{
	FScopedTransaction Transaction(LOCTEXT("MoveModifierUpTransaction", "Moving Animation Modifier Up"));
	checkf(Instance.IsValid(), TEXT("Invalid weak object ptr to modifier instance"));
	AssetUserData->Modify();

	AssetUserData->ChangeAnimationModifierIndex(Instance.Get(), -1);
	Refresh();
}

void SAnimationModifiersTab::OnMoveModifierDown(const TWeakObjectPtr<UAnimationModifier>& Instance)
{
	FScopedTransaction Transaction(LOCTEXT("MoveModifierDownTransaction", "Moving Animation Modifier Down"));
	checkf(Instance.IsValid(), TEXT("Invalid weak object ptr to modifier instance"));
	AssetUserData->Modify();

	AssetUserData->ChangeAnimationModifierIndex(Instance.Get(), 1);
	Refresh();
}

void SAnimationModifiersTab::Refresh()
{
	bDirty = true;
}

void SAnimationModifiersTab::OnBlueprintCompiled(UBlueprint* Blueprint)
{
	if (Blueprint)
	{
		Refresh();
	}
}

void SAnimationModifiersTab::OnAssetOpened(UObject* Object, IAssetEditorInstance* Instance)
{	
	RetrieveAnimationAsset();
	RetrieveModifierData();
	ModifierListView->Refresh();
}

void SAnimationModifiersTab::ApplyModifiers(const TArray<UAnimationModifier*>& Modifiers)
{
	bool bApply = true;

	TArray<UAnimSequence*> AnimSequences;
	if (AnimationSequence != nullptr)
	{
		AnimSequences.Add(AnimationSequence);
	}
	else if (Skeleton != nullptr)
	{
		// Double check with the user for applying all modifiers to referenced animation sequences for the skeleton
		bApply = FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("ApplyingSkeletonModifierPopupText", "Are you sure you want to apply the modifiers to all animation sequences referenced by the current skeleton?"), LOCTEXT("ApplyingSkeletonModifierPopupTitle", "Are you sure?")) == EAppReturnType::Yes;
		
		if (bApply)
		{
			FindAnimSequencesForSkeleton(AnimSequences);
		}
	}

	if (bApply)
	{
		UE::Anim::FApplyModifiersScope Scope;
		FScopedSlowTask SlowTaskModifier(Modifiers.Num(), LOCTEXT("SlowTaskApplyingModifier_Modifier_Start", "Applying Animation Modifier(s) to Animation Sequence(s)...") );
		SlowTaskModifier.MakeDialog();
		for (UAnimationModifier* Instance : Modifiers)
		{
			checkf(Instance, TEXT("Invalid modifier instance"));
			SlowTaskModifier.EnterProgressFrame(1.0f, FText::Format(LOCTEXT("SlowTaskApplyingModifier_Modifier_Update", "Applying Animation Modifier {0}..."), Instance->GetClass()->GetDisplayNameText()));
			bool AppliedOnAny = false;
			FScopedSlowTask SlowTaskSequences(AnimSequences.Num(), LOCTEXT("SlowTaskApplyingModifier_Sequence_Start", "Applying Animation Modifier to Animation Sequence(s)..."));
			SlowTaskSequences.MakeDialog();
			for (UAnimSequence* AnimSequence : AnimSequences)
			{
				FText AnimSequenceName = FText::FromString(IsValid(AnimSequence) ? AnimSequence->GetName() : TEXT(""));
				SlowTaskSequences.EnterProgressFrame(1.0f, FText::Format(LOCTEXT("SlowTaskApplyingModifier_Sequence_Update", "Applying Animation Modifier to Animation Sequence {0}..."), AnimSequenceName));
				ensure(!(Skeleton != nullptr) || AnimSequence->GetSkeleton() == Skeleton);
				Instance->ApplyToAnimationSequence(AnimSequence);
				AppliedOnAny = AppliedOnAny || Instance->CanRevert(AnimSequence);
			}
			if (AppliedOnAny && Skeleton)
			{
				Instance->RemoveLegacyPreviousAppliedModifierOnSkeleton(Skeleton);
			}
		}
	}
}

void SAnimationModifiersTab::RevertModifiers(const TArray<UAnimationModifier*>& Modifiers)
{
	bool bRevert = true;
	TArray<UAnimSequence*> AnimSequences;
	if (AnimationSequence != nullptr)
	{
		AnimSequences.Add(AnimationSequence);
	}
	else if (Skeleton != nullptr)
	{
		// Double check with the user for reverting all modifiers from referenced animation sequences for the skeleton
		bRevert = FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("RevertingSkeletonModifierPopupText", "Are you sure you want to revert the modifiers from all animation sequences referenced by the current skeleton?"), LOCTEXT("RevertingSkeletonModifierPopupTitle", "Are you sure?")) == EAppReturnType::Yes;

		if ( bRevert)
		{
			FindAnimSequencesForSkeleton(AnimSequences);
		}
	}

	if (bRevert)
	{
		for (UAnimationModifier* Instance : Modifiers)
		{
			checkf(Instance, TEXT("Invalid modifier instance"));
			for (UAnimSequence* AnimSequence : AnimSequences)
			{
				ensure(!(Skeleton != nullptr) || AnimSequence->GetSkeleton() == Skeleton);
				Instance->RevertFromAnimationSequence(AnimSequence);
			}

			if (Skeleton)
			{
				// Revert can not fail, thus we can always mark reverted
				Instance->RemoveLegacyPreviousAppliedModifierOnSkeleton(Skeleton);
			}
		}
	}	
}

void SAnimationModifiersTab::RetrieveModifierData()
{
	ResetModifierData();

	if (AssetUserData)
	{
		const TArray<UAnimationModifier*>& ModifierInstances = AssetUserData->GetAnimationModifierInstances();
		for (int32 ModifierIndex = 0; ModifierIndex < ModifierInstances.Num(); ++ModifierIndex)
		{
			UAnimationModifier* Modifier = ModifierInstances[ModifierIndex];
			checkf(Modifier != nullptr, TEXT("Invalid modifier ptr entry"));
			FModifierListviewItem* Item = new FModifierListviewItem();
			Item->Instance = Modifier;
			Item->Class = Modifier->GetClass();
			Item->Index = ModifierIndex;
			Item->OuterClass = AssetUserData->GetOuter()->GetClass();
			Item->OutOfDate = AnimationSequence && !Modifier->IsLatestRevisionApplied(AnimationSequence);
			ModifierItems.Add(ModifierListviewItem(Item));

			// Register a delegate for when a BP is compiled, this so we can refresh the UI and prevent issues with invalid instance data
			if (Item->Class->ClassGeneratedBy)
			{
				checkf(Item->Class->ClassGeneratedBy != nullptr, TEXT("Invalid ClassGeneratedBy value"));
				UBlueprint* Blueprint = CastChecked<UBlueprint>(Item->Class->ClassGeneratedBy);
				Blueprint->OnCompiled().AddSP(this, &SAnimationModifiersTab::OnBlueprintCompiled);
				DelegateRegisteredBlueprints.Add(Blueprint);
			}
		}

		// Refresh OutOfDate data for modifiers on Skeleton
		if (Skeleton)
		{
			TArray<FModifierListviewItem*> ModifiersToCheck; // Modifiers that are up to date (until current search progress)
			TArray<FModifierListviewItem*> NextModifiersToCheck; // Double buffer for ModifiersToCheck, store the UpToDate modifiers on this animation
			ModifiersToCheck.Reserve(ModifierItems.Num());

			// Handle backward compatibility 
			// Modifier applied in previous version will not have the applied version stored in asset registry tags
			// Exclude them in asset registry search and use the legacy OutOfDate check
			for (TSharedPtr<FModifierListviewItem>& Item : ModifierItems)
			{
				ensure(!Item->OutOfDate);
				if (Item->Instance->HasLegacyPreviousAppliedModifierOnSkeleton())
				{
					Item->OutOfDate = !Item->Instance->IsLatestRevisionApplied(Skeleton);
				}
				else
				{
					ModifiersToCheck.Add(Item.Get());
				}
			}

			if (ModifiersToCheck.IsEmpty())
			{
				return;
			}

			NextModifiersToCheck.Reserve(ModifiersToCheck.Num());

			// Modifier.OutOfDate = Any(Animation in Skeleton.Animations, Animation=>Modifier.OutOfDateOn(Animation))
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(FName("AssetRegistry"));
			IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

			FARFilter Filter;
			Filter.ClassPaths.Add(UAnimSequence::StaticClass()->GetClassPathName());
			Filter.TagsAndValues.Add(FName{"Skeleton"}) = FAssetData(Skeleton).GetExportTextName();

			AssetRegistry.EnumerateAssets(Filter, [&](const FAssetData& Asset) {
				FString Tag = Asset.GetTagValueRef<FString>(UAnimationModifier::AnimationModifiersTag);

				// First mark all modifiers to check as out of date
				// Since modifiers omitted from the tag were not applied and thus out of date
				for (FModifierListviewItem* Item : ModifiersToCheck)
				{
					Item->OutOfDate = true;
				}

				// Enumerate the current asset's tag 
				// To check the revision of each applied modifier
				// UpToDate modifiers will be add to NextModifiersToCheck to continue the search in next Asset
				NextModifiersToCheck.Empty();
				FAnimationModifierHelpers::EnumerateAnimationModifierTags(Tag, [&](FStringView Name, FGuid Revision) {
					FName ModifierName {Name};
					auto CompareModifierListViewItemName = [ModifierName](const FModifierListviewItem* Item) {
						return Item->Instance->GetFName() == ModifierName;
					};
					if (FModifierListviewItem** PtrItem = ModifiersToCheck.FindByPredicate(CompareModifierListViewItemName))
					{
						FModifierListviewItem* Item = *PtrItem;
						Item->OutOfDate = Item->Instance->GetLatestRevisionGuid() != Revision;
						if (!Item->OutOfDate)
						{
							NextModifiersToCheck.Add(Item); // Only need to continue check modifiers that is update to date
						}
					}
					return true;
				});
				// Swap the double buffer
				Swap(ModifiersToCheck, NextModifiersToCheck);

				// Break if all modifiers are out of date
				return !ModifiersToCheck.IsEmpty();
			});
		}
	}
}

void SAnimationModifiersTab::ResetModifierData()
{
	const int32 NumProcessors = (AssetUserData != nullptr) ? AssetUserData->GetAnimationModifierInstances().Num() : 0;
	DelegateRegisteredBlueprints.Empty(NumProcessors);
	ModifierItems.Empty(NumProcessors);

	for (UBlueprint* Blueprint : DelegateRegisteredBlueprints)
	{
		Blueprint->OnCompiled().RemoveAll(this);
	}
}

#undef LOCTEXT_NAMESPACE //"SAnimationModifiersTab"
