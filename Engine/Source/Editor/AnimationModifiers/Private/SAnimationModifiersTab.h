// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "EditorUndoClient.h"
#include "Input/Reply.h"
#include "SModifierListview.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API ANIMATIONMODIFIERS_API

class IAssetEditorInstance;
class IDetailsView;
class SMenuAnchor;
class UAnimSequence;
class UAnimationModifier;
class UAnimationModifiersAssetUserData;
class UBlueprint;
class UClass;
class UObject;
class USkeleton;
struct FGeometry;

class SAnimationModifiersTab : public SCompoundWidget, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SAnimationModifiersTab)
	{}	
	SLATE_ARGUMENT(TWeakPtr<class FAssetEditorToolkit>, InHostingApp)
	SLATE_END_ARGS()

	UE_API SAnimationModifiersTab();
	UE_API ~SAnimationModifiersTab();

	/** SWidget functions */
	UE_API void Construct(const FArguments& InArgs);

	/** Begin SCompoundWidget */
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	/** End SCompoundWidget */

	/** Begin FEditorUndoClient */
	UE_API virtual void PostUndo(bool bSuccess) override;
	UE_API virtual void PostRedo(bool bSuccess) override;
	/** End FEditorUndoClient */
protected:
	/** Callback for when user has picked a modifier to add */
	UE_API void OnModifierPicked(UClass* PickedClass);

	UE_API void CreateInstanceDetailsView();	

	/** UI apply all modifiers button callback */
	UE_API FReply OnApplyAllModifiersClicked();

	/** Callbacks for available modifier actions */
	UE_API void OnApplyModifier(const TArray<TWeakObjectPtr<UAnimationModifier>>& Instances);	
	UE_API void OnRevertModifier(const TArray<TWeakObjectPtr<UAnimationModifier>>& Instances);
	UE_API bool OnCanRevertModifier(const TArray<TWeakObjectPtr<UAnimationModifier>>& Instances);	
	UE_API void OnRemoveModifier(const TArray<TWeakObjectPtr<UAnimationModifier>>& Instances);
	UE_API void OnOpenModifier(const TWeakObjectPtr<UAnimationModifier>& Instance);

	UE_API void OnMoveModifierUp(const TWeakObjectPtr<UAnimationModifier>& Instance);
	UE_API void OnMoveModifierDown(const TWeakObjectPtr<UAnimationModifier>& Instance);

	/** Flags UI dirty and will refresh during the next Tick*/
	UE_API void Refresh();

	/** Callback for compiled blueprints, this ensures to refresh the UI */
	UE_API void OnBlueprintCompiled(UBlueprint* Blueprint);
	/** Callback to keep track of when an asset is opened, this is necessary for when an editor document tab is reused and this tab isn't recreated */
	UE_API void OnAssetOpened(UObject* Object, IAssetEditorInstance* Instance);

	/** Applying and reverting of modifiers */
	UE_API void ApplyModifiers(const TArray<UAnimationModifier*>& Modifiers);
	UE_API void RevertModifiers(const TArray<UAnimationModifier*>& Modifiers);

	/** Retrieves the currently opened animation asset type and modifier user data */
	UE_API void RetrieveAnimationAsset();

	/** Retrieves all animation sequences which are dependent on the current opened skeleton */
	UE_API void FindAnimSequencesForSkeleton(TArray<UAnimSequence *> &ReferencedAnimSequences);
protected:
	TWeakPtr<class FAssetEditorToolkit> HostingApp;

	/** Retrieved currently open animation asset type */
	USkeleton* Skeleton;
	UAnimSequence* AnimationSequence;
	/** Asset user data retrieved from AnimSequence or Skeleton */
	UAnimationModifiersAssetUserData* AssetUserData;
	/** List of blueprints for which a delegate was registered for OnCompiled */
	TArray<UBlueprint*> DelegateRegisteredBlueprints;
	/** Flag whether or not the UI should be refreshed  */
	bool bDirty;
protected:
	/** UI elements and data */
	TSharedPtr<IDetailsView> ModifierInstanceDetailsView;
	TArray<ModifierListviewItem> ModifierItems;
	TSharedPtr<SModifierListView> ModifierListView;
	TSharedPtr<SMenuAnchor> AddModifierCombobox;	
private:
	UE_API void RetrieveModifierData();
	UE_API void ResetModifierData();
};

#undef UE_API
