// Copyright Epic Games, Inc. All Rights Reserved.

#include "SModifierItemRow.h"

#include "Animation/Skeleton.h"
#include "AnimationModifier.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Layout/Children.h"
#include "Misc/Attribute.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/SubclassOf.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"

class STableViewBase;
struct FGeometry;
struct FPointerEvent;

void SModifierItemRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const ModifierListviewItem& Item)
{
	STableRow<ModifierListviewItem>::ConstructInternal(STableRow::FArguments(), InOwnerTableView);

	OnOpenModifier = InArgs._OnOpenModifier;
	InternalItem = Item;
	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(6.0f, 2.0f, 0.0f, 2.0f)
		[
			SNew(SImage)
			.Image(InternalItem->OuterClass == USkeleton::StaticClass() ? FAppStyle::GetBrush("ClassIcon.Skeleton") : FAppStyle::GetBrush("ClassIcon.AnimSequence"))
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(6.0f, 3.0f, 0.0f, 2.0f)
		[			
			SNew(STextBlock)
			.Text(this, &SModifierItemRow::GetInstanceText)
			.OnDoubleClicked(this, &SModifierItemRow::OnDoubleClicked)
		]
	];
}

FReply SModifierItemRow::OnDoubleClicked(const FGeometry& MyGeometry, const FPointerEvent& PointerEvent)
{
	OnOpenModifier.ExecuteIfBound(InternalItem->Instance);
	return FReply::Handled();
}

FText SModifierItemRow::GetInstanceText() const
{
	FString LabelString = InternalItem->Class->GetName();
	static const FString Postfix("_C");
	// Ensure we remove the modifier class postfix
	LabelString.RemoveFromEnd(Postfix);

	if (InternalItem->OutOfDate)
	{
		LabelString.Append(" (Out of Date)");
	}

	return FText::FromString(LabelString);
}

