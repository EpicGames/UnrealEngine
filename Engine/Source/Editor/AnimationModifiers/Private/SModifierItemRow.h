// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/BitArray.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "SModifierListview.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/STableRow.h"

class STableViewBase;
struct FGeometry;
struct FPointerEvent;

/** Listview row widget representing a single modifier instance */
class SModifierItemRow : public STableRow<ModifierListviewItem>
{
public:
	SLATE_BEGIN_ARGS(SModifierItemRow) {}
	SLATE_ARGUMENT(FOnSingleModifier, OnOpenModifier);
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const ModifierListviewItem& Item);
	FReply OnDoubleClicked(const FGeometry& MyGeometry, const FPointerEvent& PointerEvent);

protected:
	FText GetInstanceText() const;
protected:
	ModifierListviewItem InternalItem;
	FOnSingleModifier OnOpenModifier;
};
