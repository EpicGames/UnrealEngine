// Copyright Epic Games, Inc. All Rights Reserved.

#include "SModifierListview.h"

#include "AnimationModifier.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailsView.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Misc/Attribute.h"
#include "SModifierItemRow.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "UObject/UnrealNames.h"

class FUICommandList;
class ITableRow;
class STableViewBase;
class SWidget;
class UObject;

#define LOCTEXT_NAMESPACE "SModifierListview"

void SModifierListView::Construct(const FArguments& InArgs)
{
	SAssignNew(Listview, SListView<ModifierListviewItem>)
		.ListItemsSource(InArgs._Items)
		.OnGenerateRow(this, &SModifierListView::OnGenerateWidgetForList)
		.OnSelectionChanged(this, &SModifierListView::OnSelectionChanged)
		.OnContextMenuOpening(this, &SModifierListView::OnContextMenuOpening);

	ListviewItems = InArgs._Items;

	InstanceDetailsView = InArgs._InstanceDetailsView;

	OnApplyModifierDelegate = InArgs._OnApplyModifier;
	OnRevertModifierDelegate = InArgs._OnRevertModifier;
	OnCanRevertModifierDelegate = InArgs._OnCanRevertModifier;
	OnOpenModifierDelegate = InArgs._OnOpenModifier;
	OnRemoveModifierDelegate = InArgs._OnRemoveModifier;
	OnMoveUpModifierDelegate = InArgs._OnMoveUpModifier;
	OnMoveDownModifierDelegate = InArgs._OnMoveDownModifier;
	OnSelectedModifierChangedDelegate = InArgs._OnSelectedModifierChanged;
	
	this->ChildSlot
	[
		Listview->AsShared()
	];
}

TSharedRef<ITableRow> SModifierListView::OnGenerateWidgetForList(ModifierListviewItem Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SModifierItemRow, OwnerTable, Item).OnOpenModifier(OnOpenModifierDelegate);
}

void SModifierListView::OnSelectionChanged(ModifierListviewItem SelectedItem, ESelectInfo::Type SelectInfo)
{
	UObject* SelectedObject = nullptr;
	TWeakObjectPtr<UAnimationModifier> SelectedModifierInstance = nullptr;
	
	// Find object to select if an item was selected (otherwise we are de-selecting to nullptr)
	if (SelectedItem.IsValid())
	{
		SelectedObject = SelectedItem->Instance.Get();
		SelectedModifierInstance = SelectedItem->Instance;
	}

	// Set the details view to the currently selected modifier blueprint instance
	if (InstanceDetailsView.IsValid())
	{
		InstanceDetailsView->SetObject(SelectedObject);
	}
	
	OnSelectedModifierChangedDelegate.ExecuteIfBound(SelectedModifierInstance);
}

void SModifierListView::OnApplyModifier()
{
	if (OnApplyModifierDelegate.IsBound() && Listview->GetNumItemsSelected())
	{
		TArray<TWeakObjectPtr<UAnimationModifier>> Instances = GetSelectedModifierInstances();
		OnApplyModifierDelegate.Execute(Instances);
	}
}

void SModifierListView::OnRemoveModifier()
{
	if (OnRemoveModifierDelegate.IsBound() && Listview->GetNumItemsSelected())
	{
		TArray<TWeakObjectPtr<UAnimationModifier>> Instances = GetSelectedModifierInstances();
		OnRemoveModifierDelegate.Execute(Instances);
	}
}

void SModifierListView::OnOpenModifier()
{
	if (OnOpenModifierDelegate.IsBound() && ( Listview->GetNumItemsSelected() == 1))
	{
		TArray<TWeakObjectPtr<UAnimationModifier>> Instances = GetSelectedModifierInstances();
		OnOpenModifierDelegate.Execute(Instances[0]);
	}
}

void SModifierListView::OnRevertModifier()
{
	if (OnRevertModifierDelegate.IsBound() && Listview->GetNumItemsSelected())
	{
		TArray<TWeakObjectPtr<UAnimationModifier>> Instances = GetSelectedModifierInstances();
		OnRevertModifierDelegate.Execute(Instances);
	}
}

bool SModifierListView::OnCanRevertModifier()
{
	if (OnCanRevertModifierDelegate.IsBound() && Listview->GetNumItemsSelected())
	{
		TArray<TWeakObjectPtr<UAnimationModifier>> Instances = GetSelectedModifierInstances();
		return OnCanRevertModifierDelegate.Execute(Instances);
	}

	return false;
}

void SModifierListView::OnMoveUpModifier()
{
	if (OnOpenModifierDelegate.IsBound() && (Listview->GetNumItemsSelected() == 1))
	{
		TArray<TWeakObjectPtr<UAnimationModifier>> Instances = GetSelectedModifierInstances();
		OnMoveUpModifierDelegate.Execute(Instances[0]);
	}
}

void SModifierListView::OnMoveDownModifier()
{
	if (OnOpenModifierDelegate.IsBound() && (Listview->GetNumItemsSelected() == 1))
	{
		TArray<TWeakObjectPtr<UAnimationModifier>> Instances = GetSelectedModifierInstances();
		OnMoveDownModifierDelegate.Execute(Instances[0]);
	}
}

const TArray<TWeakObjectPtr<UAnimationModifier>> SModifierListView::GetSelectedModifierInstances()
{
	TArray<ModifierListviewItem> Items;
	Listview->GetSelectedItems(Items);

	TArray<TWeakObjectPtr<UAnimationModifier>> Instances;
	for (ModifierListviewItem& SelectedItem : Items)
	{		
		Instances.Add(SelectedItem->Instance);
	}

	return Instances;
}

void SModifierListView::Refresh()
{	
	Listview->RebuildList();
}

TSharedPtr<SWidget> SModifierListView::OnContextMenuOpening()
{
	const int32 NumItems = Listview->GetNumItemsSelected();
	if (NumItems >= 1)
	{
		const bool bCloseAfterSelection = true;

		FMenuBuilder MenuBuilder(bCloseAfterSelection, TSharedPtr<FUICommandList>());		
		
		MenuBuilder.BeginSection(NAME_None);
		{		
			if (Listview->GetNumItemsSelected() == 1 && OnOpenModifierDelegate.IsBound())
			{
				MenuBuilder.AddMenuEntry(LOCTEXT("OpenModifierLabel", "Open Blueprint"), LOCTEXT("OpenModifierToolTip", "Open selected Modifier Blueprint"), FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Blueprint"), FUIAction(FExecuteAction::CreateSP(this, &SModifierListView::OnOpenModifier)));
			}

			const FText ApplyLabel = FText::FormatOrdered(LOCTEXT("ApplyModifierLabel", "Apply {0}|plural(one=Modifier,other=Modifiers)"), NumItems);
			const FText ApplyTooltip = FText::FormatOrdered(LOCTEXT("ApplyModifierToolTip", "Apply selected {0}|plural(one=Modifier,other=Modifiers)"), NumItems);
			if (OnApplyModifierDelegate.IsBound())
			{
				MenuBuilder.AddMenuEntry(ApplyLabel, ApplyTooltip, FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Redo"), FUIAction(FExecuteAction::CreateSP(this, &SModifierListView::OnApplyModifier)));
			}
			
			const FText RevertLabel = FText::FormatOrdered(LOCTEXT("ApplyRevertLabel", "Revert {0}|plural(one=Modifier,other=Modifiers)"), NumItems);
			const FText RevertTooltip = FText::FormatOrdered(LOCTEXT("ApplyRevertToolTip", "Revert selected {0}|plural(one=Modifier,other=Modifiers)"), NumItems);
			if (OnRevertModifierDelegate.IsBound())
			{
				MenuBuilder.AddMenuEntry(RevertLabel, RevertTooltip, FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Undo"), FUIAction(FExecuteAction::CreateSP(this, &SModifierListView::OnRevertModifier), FCanExecuteAction::CreateSP(this, &SModifierListView::OnCanRevertModifier)));
			}

			const FText RemoveLabel = FText::FormatOrdered(LOCTEXT("RemoveModifierLabel", "Remove {0}|plural(one=Modifier,other=Modifiers)"), NumItems);
			const FText RemoveTooltip = FText::FormatOrdered(LOCTEXT("RemoveModifierToolTip", "Remove selected {0}|plural(one=Modifier,other=Modifiers)"), NumItems);
			if (OnRemoveModifierDelegate.IsBound())
			{
				MenuBuilder.AddMenuEntry(RemoveLabel, RemoveTooltip, FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions.Delete"), FUIAction(FExecuteAction::CreateSP(this, &SModifierListView::OnRemoveModifier)));
			}
		}
		MenuBuilder.EndSection();

		if (Listview->GetNumItemsSelected() == 1)
		{ 
			MenuBuilder.BeginSection(NAME_None);
			{
				if (OnMoveUpModifierDelegate.IsBound())
				{
					MenuBuilder.AddMenuEntry(LOCTEXT("MoveUpModifierLabel", "Move Up"), LOCTEXT("MoveUpModifierToolTip", "Move selected Modifier Up in list"), FSlateIcon(), FUIAction(FExecuteAction::CreateSP(this, &SModifierListView::OnMoveUpModifier), FCanExecuteAction::CreateSP(this, &SModifierListView::CanMoveSelectedItemUp)));
				}
				
				if (OnMoveDownModifierDelegate.IsBound())
				{
					MenuBuilder.AddMenuEntry(LOCTEXT("MoveDownModifierLabel", "Move Down"), LOCTEXT("MoveDownModifierToolTip", "Move selected Modifier Down in list"), FSlateIcon(), FUIAction(FExecuteAction::CreateSP(this, &SModifierListView::OnMoveDownModifier), FCanExecuteAction::CreateSP(this, &SModifierListView::CanMoveSelectedItemDown)));
				}
			}
			MenuBuilder.EndSection();
		}

		return MenuBuilder.MakeWidget();
	}
	
	return TSharedPtr<SWidget>();
}

bool SModifierListView::CanMoveSelectedItemUp()
{
	bool bCanMove = false;
	if (Listview->GetNumItemsSelected() == 1)
	{
		TArray<ModifierListviewItem> Items;
		Listview->GetSelectedItems(Items);
		bCanMove = (Items[0]->Index > 0);
	}

	return bCanMove;
}

bool SModifierListView::CanMoveSelectedItemDown()
{
	bool bCanMove = false;
	if (Listview->GetNumItemsSelected() == 1)
	{
		TArray<ModifierListviewItem> Items;
		Listview->GetSelectedItems(Items);
		bCanMove = (Items[0]->Index < (ListviewItems->Num() - 1));
	}

	return bCanMove;
}

#undef LOCTEXT_NAMESPACE // "SModifierListview"

