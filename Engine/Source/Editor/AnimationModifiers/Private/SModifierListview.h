// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class IDetailsView;
class ITableRow;
class STableViewBase;
class SWidget;
class UAnimationModifier;
class UClass;

DECLARE_DELEGATE_OneParam(FOnModifierArray, const TArray<TWeakObjectPtr<UAnimationModifier>>&);
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnCanModifierArray, const TArray<TWeakObjectPtr<UAnimationModifier>>&);
DECLARE_DELEGATE_OneParam(FOnSingleModifier, const TWeakObjectPtr<UAnimationModifier>&);

/** Data representation of a modifier in the listview */
struct FModifierListviewItem
{
	TSubclassOf<UAnimationModifier> Class;
	TWeakObjectPtr<UAnimationModifier> Instance;
	int32 Index;
	UClass* OuterClass;
	bool OutOfDate;
};

typedef TSharedPtr<FModifierListviewItem> ModifierListviewItem;

class SModifierListView : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SModifierListView) 
		: _Items(nullptr) {}
	SLATE_ARGUMENT(TArray<ModifierListviewItem>*, Items)
	SLATE_ARGUMENT(TSharedPtr<IDetailsView>, InstanceDetailsView)
	SLATE_ARGUMENT(FOnModifierArray, OnApplyModifier);
	SLATE_ARGUMENT(FOnCanModifierArray, OnCanRevertModifier)
	SLATE_ARGUMENT(FOnModifierArray, OnRevertModifier);
	SLATE_ARGUMENT(FOnModifierArray, OnRemoveModifier);
	SLATE_ARGUMENT(FOnSingleModifier, OnOpenModifier);
	SLATE_ARGUMENT(FOnSingleModifier, OnMoveUpModifier);
	SLATE_ARGUMENT(FOnSingleModifier, OnMoveDownModifier);
	SLATE_EVENT(FOnSingleModifier, OnSelectedModifierChanged);
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	
	TSharedRef<ITableRow> OnGenerateWidgetForList(ModifierListviewItem Item, const TSharedRef<STableViewBase>& OwnerTable);
	void OnSelectionChanged(ModifierListviewItem SelectedItem, ESelectInfo::Type SelectInfo);
	
	/** Context menu callbacks for modifier actions */
	void OnApplyModifier();
	void OnRemoveModifier();
	void OnOpenModifier();
	void OnRevertModifier();
	bool OnCanRevertModifier();
	void OnMoveUpModifier();
	void OnMoveDownModifier();

	/** Retrieve currently select Modifier instances */
	const TArray<TWeakObjectPtr<UAnimationModifier>> GetSelectedModifierInstances();

	/** Refreshes the listview panel */
	void Refresh();

	/** Generate the context menu widget when requested */
	TSharedPtr<SWidget> OnContextMenuOpening();

protected:
	/** UI elements and listview data */
	TSharedPtr<SListView<ModifierListviewItem>> Listview;
	TArray<ModifierListviewItem>* ListviewItems;
	TSharedPtr<IDetailsView> InstanceDetailsView;

	/** Delegates back to SAnimationModifiersTab functionality */
	FOnModifierArray OnApplyModifierDelegate;
	FOnModifierArray OnRevertModifierDelegate;
	FOnCanModifierArray OnCanRevertModifierDelegate;
	FOnModifierArray OnRemoveModifierDelegate;
	FOnSingleModifier OnOpenModifierDelegate;
	FOnSingleModifier OnMoveUpModifierDelegate;
	FOnSingleModifier OnMoveDownModifierDelegate;
	FOnSingleModifier OnSelectedModifierChangedDelegate;
	
	/** Check whether or not selected modifier can moved in either direction */
	bool CanMoveSelectedItemUp();
	bool CanMoveSelectedItemDown();
};
