// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimEnums.h"
#include "AnimationBlueprintLibrary.h"
#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "HAL/PlatformMisc.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Guid.h"
#include "Misc/MessageDialog.h"
#include "Misc/Optional.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "AnimationModifier.generated.h"

#define UE_API ANIMATIONMODIFIERS_API

class FArchive;
class FName;
class UAnimSequence;
class UClass;
class USkeleton;
struct FFrame;
struct FObjectKey;

UCLASS(MinimalAPI, Blueprintable, config = Editor, defaultconfig)
class UAnimationModifier : public UObject
{
	GENERATED_BODY()

	friend class FAnimationModifierDetailCustomization;
public:
	UE_API UAnimationModifier();

	/** Applying and reverting the modifier for the given Animation Sequence */
	UE_API void ApplyToAnimationSequence(UAnimSequence* AnimSequence) const;
	UE_API void RevertFromAnimationSequence(UAnimSequence* AnimSequence) const;

	/** Executed when the Animation is initialized (native event for debugging / testing purposes) */
	UFUNCTION(BlueprintNativeEvent)
	UE_API void OnApply(UAnimSequence* AnimationSequence);
	virtual void OnApply_Implementation(UAnimSequence* AnimationSequence) {}
	
	UFUNCTION(BlueprintNativeEvent)
	UE_API void OnRevert(UAnimSequence* AnimationSequence);
	virtual void OnRevert_Implementation(UAnimSequence* AnimationSequence) {}

	/** Returns whether or not this modifier can be reverted, which means it will have to been applied to the given AnimationSequence */
	UE_API bool CanRevert(IInterface_AssetUserData* AssetUserDataInterface) const;

	/** Whether or not the latest compiled version of the blueprint is applied for this instance */
	UE_API bool IsLatestRevisionApplied(IInterface_AssetUserData* AssetUserDataInterface) const;

	//! @brief Get the latest revision GUID of this modifier class
	UE_API FGuid GetLatestRevisionGuid() const;

	//! @brief Get Asset Registry tags for applied modifiers from _skeleton_
	//! @param[out] OutTags AnimationModifiers = %PATH%=%REVISION%;...
	static UE_API void GetAssetRegistryTagsForAppliedModifiersFromSkeleton(FAssetRegistryTagsContext Context);

	static UE_API const FName AnimationModifiersTag;
	static constexpr TCHAR AnimationModifiersDelimiter = ';';
	static constexpr TCHAR AnimationModifiersAssignment = '=';

	//! @brief Check if this Modifier On Skeleton has PreviouslyAppliedModifier_DEPRECATED from previous version
	bool HasLegacyPreviousAppliedModifierOnSkeleton() const { return GetLegacyPreviouslyAppliedModifierForModifierOnSkeleton() != nullptr; }
	//! @brief Mark this modifier on skeleton as reverted (affect CanRevert, IsLatestRevisionApplied)
	UE_API void RemoveLegacyPreviousAppliedModifierOnSkeleton(USkeleton* Skeleton);

	// Begin UObject Overrides
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
	// End UObject Overrides

	/** If this is set to true then the animation modifier will call it's reapply function after any change made to the owning asset. */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bReapplyPostOwnerChange = false;

protected:
	// Derived class accessors to skeleton and anim sequence 
	UE_API const UAnimSequence* GetAnimationSequence();
	UE_API const USkeleton* GetSkeleton();

	/** Used for natively updating the revision GUID, fairly basic and relies on config files currently */
	UE_API virtual int32 GetNativeClassRevision() const;

	/** Checks if the animation data has to be re-baked / compressed and does so */
	UE_API void UpdateCompressedAnimationData();

	/** Updating of blueprint and native GUIDs*/
	UE_API void UpdateRevisionGuid(UClass* ModifierClass);
	UE_API void UpdateNativeRevisionGuid();

	UE_API void ExecuteOnRevert(UAnimSequence* InAnimSequence);
	UE_API void ExecuteOnApply(UAnimSequence* InAnimSequence);
	
	/** Applies all instances of the provided Modifier class to its outer Animation Sequence*/
	static UE_API void ApplyToAll(TSubclassOf<UAnimationModifier> ModifierSubClass, bool bForceApply = true);
	static UE_API void LoadModifierReferencers(TSubclassOf<UAnimationModifier> ModifierSubClass);
private:
	// These value were not set during OnRevert, prefer to use the input from OnApply() or OnRevert()
	UAnimSequence* CurrentAnimSequence = nullptr;
	USkeleton* CurrentSkeleton = nullptr;

	//! @brief Try get the applied modifier instance (associated with this modifier) on the animation sequence, null if not applied on the sequence
	UE_API UAnimationModifier* GetAppliedModifier(IInterface_AssetUserData* AssetUserDataInterface) const;

	//! @brief Set the applied modifier instance on the Animation Sequence
	UE_API void SetAppliedModifier(TScriptInterface<IInterface_AssetUserData> AssetUserDataInterface, UAnimationModifier* AppliedModifier) const;
	//! @brief Remove the applied modifier instance (associated with this modifier) from the Animation Sequence
	//! @return The modifier instance removed
	UE_API UAnimationModifier* FindAndRemoveAppliedModifier(TScriptInterface<IInterface_AssetUserData> AssetUserDataInterface) const;
	//! @brief Get the applied modifier instance for modifier on skeleton read from _legacy_ version
	//! @note This function is only intended for backward compatibility with existing asset
	//!	@return PreviouslyAppliedModifier_DEPRECATED on Modifier on Skeleton, _before_ upgrade
	UE_API UAnimationModifier* GetLegacyPreviouslyAppliedModifierForModifierOnSkeleton() const;


	// - On class-default-objects, this is the latest revision GUID
	// - On applied modifier instances, this is the revision applied
	// - Not used for other instances
	UPROPERTY(/*VisibleAnywhere for testing, Category = Revision*/)
	FGuid RevisionGuid;

	// This indicates whether or not the modifier is newer than what has been applied
	UPROPERTY(/*VisibleAnywhere for testing, Category = Revision */)
	FGuid AppliedGuid_DEPRECATED;

	// This holds the latest value returned by UpdateNativeRevisionGuid during the last PostLoad (changing this value will invalidate the GUIDs for all instances)
	UPROPERTY(config)
	int32 StoredNativeRevision;

	/** Serialized version of the modifier that has been previously applied to the Animation Asset */
	UPROPERTY()
	TObjectPtr<UAnimationModifier> PreviouslyAppliedModifier_DEPRECATED = nullptr;
};

namespace UE
{
	namespace Anim
	{
		// RAII object to determine how Animation Modifier warning/errors are handled (ignore, user dialog, etc.)
		struct FApplyModifiersScope
		{
			friend UAnimationModifier;

			enum ESuppressionMode : uint8
			{
				// Do not change the error handling mode on this scope
				// Use the mode set by parent scope or default (ShowDialog)
				NoChange,
				// Suppress error dialogs
				// Suppress warnings dialogs, always apply modifiers
				// No user interaction required
				SuppressWarningAndError,
				// Show error dialogs
				// Suppress warnings dialogs, always apply modifiers
				SuppressWarning,
				// Show warning and error dialogs for first encounter
				// Error dialog for each modifier class will only be showed once
				ShowDialog,
				// Always show the error or warning dialog
				// Default behavior when no scope was open
				ForceDialog,
				// Suppress error dialogs
				// Suppress warnings dialogs, always revert modifiers
				// No user interaction required
				RevertAtWarning,
			};

			FApplyModifiersScope(const FApplyModifiersScope&) = delete;
			explicit FApplyModifiersScope(ESuppressionMode Mode = NoChange)
			{
				Open(Mode);
			}

			~FApplyModifiersScope()
			{
				Close();
			}

			static bool IsScopePending() { return !ScopeModeStack.IsEmpty(); }
protected:
			/** Determine how to handle an Animation Modifier error, and execute accordingly */
			static UE_API void HandleError(const UAnimationModifier* Modifier, const FText& Message, const FText& Title);

			/** Determine how to handle an Animation Modifier warning, and execute accordingly. Returns whether or not the warning was handled (true) or warrants reverting the applied Animation Modifier (false) */
			static UE_API bool HandleWarning(const UAnimationModifier* Modifier, const FText& Message, const FText& Title);

private:
			// Open a scope to control error handling when batch applying animation modifiers
			static UE_API ESuppressionMode Open(ESuppressionMode Mode = NoChange);
			// Close the most recent scope
			static UE_API void Close();

			static UE_API ESuppressionMode CurrentMode();

			/** Errors already acknowledged */
			static UE_API TSet<FObjectKey> ErrorResponse;
			/** Warnings already ignored or treated as error */
			static UE_API TMap<FObjectKey, EAppReturnType::Type> WarningResponse;
			/** Error handle mode stack for scopes */
			static UE_API TArray<ESuppressionMode, TInlineAllocator<4>> ScopeModeStack;
		};
	}
}

#undef UE_API
