// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Engine/AssetUserData.h"
#include "HAL/Platform.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "AnimationModifier.h"

#include "AnimationModifiersAssetUserData.generated.h"

#define UE_API ANIMATIONMODIFIERS_API

class FArchive;
class UObject;
template <typename T> struct TObjectPtr;

/** Asset user data which can be added to a USkeleton or UAnimSequence to keep track of Animation Modifiers */
UCLASS(MinimalAPI)
class UAnimationModifiersAssetUserData : public UAssetUserData
{
	GENERATED_BODY()

	friend class SAnimationModifiersTab;
	friend class SAnimationModifierContentBrowserWindow;
	friend class SRemoveAnimationModifierContentBrowserWindow;
	friend class FAnimationModifiersModule;
	friend class UAnimationModifier;
public:
	UE_API UAnimationModifiersAssetUserData(const FObjectInitializer& ObjectInitializer);
	UE_API const TArray<UAnimationModifier*>& GetAnimationModifierInstances() const;
	
	UFUNCTION(BlueprintCallable, Category=Animation, meta = (ScriptMethod))
	static UE_API bool AddAnimationModifierOfClass(UAnimSequenceBase* AnimationSequenceBase, TSubclassOf<UAnimationModifier> InModifierClass);
protected:	 
	/** Begin UAssetUserData overrides */
	UE_API virtual void PostLoad() override;
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;	
	UE_API virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeOwner(const FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	/** End UAssetUserData overrides */

	UE_API void AddAnimationModifier(UAnimationModifier* Instance);
	UE_API void RemoveAnimationModifierInstance(UAnimationModifier* Instance);
	UE_API void ChangeAnimationModifierIndex(UAnimationModifier* Instance, int32 Direction);
private:
	UE_API void RemoveInvalidModifiers();
protected:
	UPROPERTY(BlueprintReadOnly, Category="Animation Modifiers")
	TArray<TObjectPtr<UAnimationModifier>> AnimationModifierInstances;

	// Animation modifiers APPLIED on the owning animation sequence
	// 
	// - Key = Modifier in AnimationModifierInstances of owning animation sequence or skeleton
	// - Value = Modifier instance applied
	// 
	// Applied modifier instances stores (Properties, RevisionGuid, ...) at Apply Time
	// Which are not visible or editable from user interface
	// 
	// In contrast, modifiers in AnimationModifierInstances 
	// are objects displayed at the Animation Data Modifier window
	// Where Properties are displayed for user editing anytime
	// 
	// Note, Modifier on Skeleton (MoS) applied instances are stored on each 
	// animation sequence's asset user data, instead of the skeleton's
	// this design is important to enable applied modifiers can be reverted for each sequence
	// also ensure applying MoS (when [re]importing animation) will not affect the skeleton asset
	//
	// The only time MoS stores an applied instance here is upgrading from previous version
	// The "UAnimationModifier::PreviouslyAppliedModifier_DEPRECATED" data will be migrated here
	// To support proper revert for them
	// For more compatibility handling, check UAnimationModifier::PostLoad() and GetAppliedModifier() 
	UPROPERTY(BlueprintReadOnly, Category="Animation Modifiers")
	TMap<FSoftObjectPath, TObjectPtr<UAnimationModifier>> AppliedModifiers;
};

#undef UE_API
