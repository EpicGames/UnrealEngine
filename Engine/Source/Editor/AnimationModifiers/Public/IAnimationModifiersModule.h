// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Containers/Array.h"

class UAnimSequence;

class IAnimationModifiersModule : public IModuleInterface
{
public:
	/** Shows a new modal dialog allowing the user to setup Animation Modifiers to be added for all AnimSequences part of InSequences */
	virtual void ShowAddAnimationModifierWindow(const TArray<UAnimSequence*>& InSequences) = 0;

	/** Shows a new modal dialog allowing the user to setup Animation Modifiers to be removed for all AnimSequences part of InSequences */
	virtual void ShowRemoveAnimationModifierWindow(const TArray<UAnimSequence*>& InSequences) = 0;

	/** Apply animation modifiers contained by provided Animation Sequences to its respective outers */
	virtual void ApplyAnimationModifiers(const TArray<UAnimSequence*>& InSequences, bool bForceApply = true) = 0;
};
